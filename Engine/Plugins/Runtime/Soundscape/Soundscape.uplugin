{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Soundscape Plugin",
	"Description": "A Dynamic Ambient Sound System",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "Soundscape",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "SoundscapeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}