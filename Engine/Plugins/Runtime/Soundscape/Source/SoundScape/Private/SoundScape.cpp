// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundScape.h"
#include "SoundScapePalette.h"
#include "SoundscapeColor.h"

bool USoundscapeBPFunctionLibrary::SpawnSoundscapeColor(UObject* WorldContextObject, class USoundscapeColor* SoundscapeColorIn, UActiveSoundscapeColor*& ActiveSoundscapeColor)
{
	// Verify that the input SoundscapeElement is not null
	if (SoundscapeColorIn && WorldContextObject)
	{
		// Create a new ActiveSoundscapeColor
		ActiveSoundscapeColor = NewObject<UActiveSoundscapeColor>(WorldContextObject);

		// Initialize parameters
		ActiveSoundscapeColor->SetParameterValues(SoundscapeColorIn);

#if WITH_EDITOR
		// Bind to delegate for live update from the SoundscapeColor Editor
		ActiveSoundscapeColor->BindToParameterChangeDelegate(SoundscapeColorIn);
#endif

		return true;
	}

	return false;
}

bool USoundscapeBPFunctionLibrary::SpawnSoundscapePalette(UObject* WorldContextObject, class USoundscapePalette* SoundscapePaletteIn, UActiveSoundscapePalette*& ActiveSoundscapePaletteOut)
{
	// Verify that the input SoundscapeElement is not null
	if (SoundscapePaletteIn && WorldContextObject)
	{
		// Create a new ActiveSoundscapeColor
		ActiveSoundscapePaletteOut = NewObject<UActiveSoundscapePalette>(WorldContextObject);

		// Initialize parameters
		ActiveSoundscapePaletteOut->InitializeSettings(WorldContextObject, SoundscapePaletteIn);

		return true;
	}

	return false;
}
