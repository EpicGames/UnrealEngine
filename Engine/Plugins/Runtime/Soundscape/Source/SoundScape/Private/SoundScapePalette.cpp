// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundScapePalette.h"
#include "SoundscapeColor.h"

USoundscapePalette::USoundscapePalette()
{
}

void USoundscapePalette::PostLoad()
{
	Super::PostLoad();
}

void USoundscapePalette::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
}

void UActiveSoundscapePalette::InitializeSettings(UObject* WorldContextObject, USoundscapePalette* SoundscapePalette)
{
	if (SoundscapePalette && WorldContextObject)
	{
		// Set up Soundscape Colors
		for (FSoundscapePaletteColor& SoundscapePaletteColor : SoundscapePalette->Colors)
		{
			if (USoundscapeColor* SoundscapeColor = SoundscapePaletteColor.SoundscapeColor)
			{
				// Verify that the input SoundscapeElement is not null
				if (SoundscapeColor)
				{
					// Create a new ActiveSoundscapeColor
					UActiveSoundscapeColor* ActiveSoundscapeColor = NewObject<UActiveSoundscapeColor>(WorldContextObject);

					//
					ActiveSoundscapeColor->SetParameterValues(SoundscapeColor);

#if WITH_EDITOR
					ActiveSoundscapeColor->BindToParameterChangeDelegate(SoundscapeColor);
#endif //WITH_EDITOR
					ActiveSoundscapeColor->VolumeMod = SoundscapePaletteColor.ColorVolume;
					ActiveSoundscapeColor->PitchMod = SoundscapePaletteColor.ColorPitch;
					ActiveSoundscapeColor->FadeInMin = SoundscapePaletteColor.ColorFadeIn;

					// Playback stop values
					ActiveSoundscapeColor->FadeOutMin = SoundscapePaletteColor.ColorFadeOut;

					// Add to list
					ActiveSoundscapeColors.Add(ActiveSoundscapeColor);
				}

			}
		}
	}
}

void UActiveSoundscapePalette::Play()
{
	for (UActiveSoundscapeColor* ActiveSoundscapeColor : ActiveSoundscapeColors)
	{
		if (ActiveSoundscapeColor)
		{
			ActiveSoundscapeColor->PlayNative();
		}
	}
}

void UActiveSoundscapePalette::Stop()
{
	for (UActiveSoundscapeColor* ActiveSoundscapeColor : ActiveSoundscapeColors)
	{
		if (ActiveSoundscapeColor)
		{
			ActiveSoundscapeColor->StopNative();
		}
	}

}

