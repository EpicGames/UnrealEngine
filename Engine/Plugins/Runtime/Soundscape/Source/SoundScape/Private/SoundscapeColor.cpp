// Copyright Epic Games, Inc. All Rights Reserved.


#include "SoundscapeColor.h"
#include "Engine/GameInstance.h"
#include "Engine/HitResult.h"
#include "AudioDevice.h"
#include "TimerManager.h"
#include "SoundscapeSubsystem.h"

USoundscapeColor::USoundscapeColor()
	: VolumeBase(1.0f)
	, PitchBase(1.0f)
{
}

void USoundscapeColor::PostLoad()
{
	Super::PostLoad();
}

void USoundscapeColor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
}

#if WITH_EDITORONLY_DATA
void USoundscapeColor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Broadcast changes to the properties so instances can update
	OnSoundscapeColorParameterChanges.Broadcast(this);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UActiveSoundscapeColorVoice::BeginDestroy()
{
	if (UWorld* World = GetWorld())
	{
		// Clear out all timers
		World->GetTimerManager().ClearTimer(TimerHandle);
	}

	Super::BeginDestroy();
}

void UActiveSoundscapeColorVoice::StopLimitedDurationVoice(float FadeOutTime)
{
	if (AudioComponent)
	{
		AudioComponent->FadeOut(FadeOutTime, 0.0f);
	}
}

void UActiveSoundscapeColor::BeginDestroy()
{
	if (UWorld* World = GetWorld())
	{
		// Clear out all timers
		World->GetTimerManager().ClearTimer(UpdateTimer);
	}

	Super::BeginDestroy();
}

void UActiveSoundscapeColor::SetParameterValues(const USoundscapeColor* SoundscapeColor)
{
	if (SoundscapeColor)
	{
		Sound = SoundscapeColor->Sound;

		VolumeBase = SoundscapeColor->VolumeBase;
		PitchBase = SoundscapeColor->PitchBase;

		ModulationBehavior = SoundscapeColor->ModulationBehavior;
		PlaybackBehavior = SoundscapeColor->PlaybackBehavior;
		SpawnBehavior = SoundscapeColor->SpawnBehavior;
	}
}

#if WITH_EDITOR
void UActiveSoundscapeColor::BindToParameterChangeDelegate(USoundscapeColor* SoundscapeColor)
{

#if WITH_EDITORONLY_DATA
	SoundscapeParameterChangeDelegate.BindUFunction(this, TEXT("OnSoundscapeColorParameterChange"));
	SoundscapeColor->OnSoundscapeColorParameterChanges.Add(SoundscapeParameterChangeDelegate);
#endif
}
#endif

#if WITH_EDITOR
void UActiveSoundscapeColor::OnSoundscapeColorParameterChange(const USoundscapeColor* SoundscapeColor)
{
	SetParameterValues(SoundscapeColor);
}
#endif

void UActiveSoundscapeColor::PlayNative()
{
	if (bIsPlaying)
	{
		// Early out if we're already playing
		return;
	}

	// Call internal play
	StartPlaying();

}

void UActiveSoundscapeColor::StopNative()
{
	if (bIsPlaying == false)
	{
		// Early out if we've already stopped
		return;
	}

	// Call internal stop
	StopPlaying();

}

void UActiveSoundscapeColor::Play(float ColorVolume, float ColorPitch, float ColorFadeInTime)
{
	if (bIsPlaying)
	{
		// Early out if we're already playing
		return;
	}

	// Cache play values
	VolumeMod = ColorVolume;
	PitchMod = ColorPitch;
	FadeInMin = ColorFadeInTime;

	// Call internal play
	StartPlaying();
}

void UActiveSoundscapeColor::Stop(float ColorFadeOutTime)
{
	if (bIsPlaying == false)
	{
		// Early out if we've already stopped
		return;
	}

	// Cache stop values
	FadeOutMin = ColorFadeOutTime;

	// Call internal stop
	StopPlaying();
}

bool UActiveSoundscapeColor::IsPlaying()
{
	return bIsPlaying;
}

void UActiveSoundscapeColor::UpdateSoundscapeColor()
{
	if (bIsPlaying)
	{
		Update();
	}
}

FTraceHandle UActiveSoundscapeColor::SpawnSoundByTrace()
{
	// Bind Trace Delegate to this object to call OnTraceCompleted
	TraceDelegate.BindUObject(this, &UActiveSoundscapeColor::OnTraceCompleted);

	if (UWorld* World = GetWorld())
	{
		// Get Trace start and end vectors
		FVector TraceStart = NewSoundParams.Location;
		FVector TraceEnd = TraceStart + (SpawnBehavior.TraceDistance * SpawnBehavior.TraceVector);

		// Flag that we are now tracing (won't trigger another trace until we're done)
		bTracing = true;

		// Kick off async trace, return the handle
		return World->AsyncLineTraceByChannel(
			EAsyncTraceType::Single
			, TraceStart
			, TraceEnd
			, SpawnBehavior.TraceChannel
			, FCollisionQueryParams::DefaultQueryParam
			, FCollisionResponseParams::DefaultResponseParam
			, &TraceDelegate
		);
	}

	return FTraceHandle();
}

void UActiveSoundscapeColor::OnTraceCompleted(const FTraceHandle& Handle, FTraceDatum& Data)
{
	// Set tracing flag to false now that we've completed our async trace
	bTracing = false;

	// Check if there are Data Hit Results
	if (Data.OutHits.Num() > 0)
	{
		// Get first result
		if (Data.OutHits[0].bBlockingHit)
		{
			// Update sound location
			NewSoundParams.Location = Data.OutHits[0].ImpactPoint;

			if (EvaluateColorPointDensity())
			{
				// Spawn new sound at updated location
				SpawnSoundscapeColorVoice();
			}
		}
	}
}


void UActiveSoundscapeColor::StartPlaying()
{
	// Update state
	bIsPlaying = true;
	bFirstSpawn = true;

	// Set up for first Timer delay
	float FirstDelayTime = 0.0001f;

	if (SpawnBehavior.bDelayFirstSpawn)
	{
		// If we delay the first spawn, then 
		FirstDelayTime = FMath::FRandRange(SpawnBehavior.MinFirstSpawnDelay, SpawnBehavior.MaxFirstSpawnDelay);
	}
	
	if (UWorld* World = GetWorld())
	{
			
		World->GetTimerManager().SetTimer(UpdateTimer, this, &UActiveSoundscapeColor::UpdateSoundscapeColor, FirstDelayTime);
	}
}

void UActiveSoundscapeColor::StopPlaying()
{
	// Stop with appropriate fade time
	for (UActiveSoundscapeColorVoice* SoundscapeColorVoice : SoundscapeColorVoicePool)
	{
		if (SoundscapeColorVoice)
		{

			if (UAudioComponent* VoiceAudioComponent = SoundscapeColorVoice->AudioComponent)
			{
				if (EAudioComponentPlayState::Stopped != VoiceAudioComponent->GetPlayState())
				{
					VoiceAudioComponent->FadeOut(FadeOutMin, 0.0f);
				}

				if (UWorld* World = GetWorld())
				{
					// Clear out all timers
					World->GetTimerManager().ClearTimer(SoundscapeColorVoice->TimerHandle);
				}
				
			}
		}
	}

	bIsPlaying = false;
}

void UActiveSoundscapeColor::Update()
{
	UWorld* World = GetWorld();
	bool bNeedToSpawnSound = false;
	bVoiceInPoolFree = false;
	int32 ActiveVoicesPlaying = 0;

	FVector ListenerLocation = FVector::ZeroVector;
	FVector ListenerForward = FVector::ZeroVector;
	FVector ListenerUp = FVector::ZeroVector;
	
	if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
	{
		// Get available Listener Proxies
		TArray<FListenerProxy>& ListenerProxies = AudioDevice->ListenerProxies;

		// TODO: Handle Split Screen
		if (ListenerProxies.Num())
		{
			// Get Location and Direction from Listener
			const FTransform ListenerTransform = ListenerProxies[0].Transform;
			ListenerLocation = ListenerTransform.GetLocation();
			ListenerForward = ListenerTransform.GetRotation().GetForwardVector();
			ListenerUp = ListenerTransform.GetRotation().GetUpVector();
		}
	}

	// Free up old voices
	for (UActiveSoundscapeColorVoice* SoundscapeColorVoice : SoundscapeColorVoicePool)
	{
		if (SoundscapeColorVoice)
		{
			
			if (UAudioComponent* VoiceAudioComponent = SoundscapeColorVoice->AudioComponent)
			{
				// Get current distance from listener
				FVector DistanceRelativeToListener = VoiceAudioComponent->GetComponentTransform().GetLocation() - ListenerLocation;
				DistanceRelativeToListener.Z = (SpawnBehavior.bCullByMaxSpawnDistance && SpawnBehavior.bIgnoreZAxisWhenCullingByMaxDistance) ? 0.0f : DistanceRelativeToListener.Z;
				const float DistanceFromListener = DistanceRelativeToListener.Length();

				// If Color Voice is out of range by more than 1% of the max distance, stop the sound
				if(DistanceFromListener > (SpawnBehavior.MaxSpawnDistance * 1.01f) && SpawnBehavior.bCullByMaxSpawnDistance)
				{
					const float FadeTime = FMath::Max(1.0f, ModulationBehavior.MinFadeOutTime);
					VoiceAudioComponent->FadeOut(FadeTime, 0.0f);
				}

				if (EAudioComponentPlayState::Stopped == VoiceAudioComponent->GetPlayState())
				{
					// Voice is not in use, mark it as free to use
					SoundscapeColorVoice->bFree = true;

					// Flag that a voice is available
					bVoiceInPoolFree = true;
				}
				else
				{
					// Voice is in use, increment active voice count
					++ActiveVoicesPlaying;
				}
			}
		}
	}

	// Determine if conditions are right to play a new sound
	if (ActiveVoicesPlaying < SpawnBehavior.MaxNumberOfSpawnedElements)
	{
		bNeedToSpawnSound = true;
	}

	// Set up timer if needed
	if (SpawnBehavior.bContinuouslyRespawn)
	{
		// Set random range spawn time, with a minimum value of 0.0001f (which is basically next frame)
		float SpawnDelayTime = FMath::Max(0.0001f, FMath::FRandRange(SpawnBehavior.MinSpawnDelay, SpawnBehavior.MaxSpawnDelay));

		// Set off timer again
		if (World)
		{
			World->GetTimerManager().SetTimer(UpdateTimer, this, &UActiveSoundscapeColor::UpdateSoundscapeColor, SpawnDelayTime);
		}
	}

	// Play Sound if needed and valid sound available
	if (Sound && bNeedToSpawnSound && World)
	{
		// Calculate Spawn Parameters
		CalculateSpawnParams();

		// Check to see if Position By Trace
		if (SpawnBehavior.bPositionByTrace == false && EvaluateColorPointDensity())
		{
			//Spawn Color Voice
			SpawnSoundscapeColorVoice();
		}
		else if (bTracing == false)
		{
			//If we're not currently mid-trace, spawn sound by trace
			TraceHandle = SpawnSoundByTrace();
		}
	}

	// No longer the first time
	bFirstSpawn = false;
}

void UActiveSoundscapeColor::CalculateSpawnParams()
{
	if (UWorld* World = GetWorld())
	{
		// Play Sound if needed and valid sound available
		if (Sound)
		{
			// Get the new sound's volume
			float NewRandVolume = ModulationBehavior.bRandomizeVolume ? FMath::FRandRange(ModulationBehavior.VolumeMin, ModulationBehavior.VolumeMax) : 1.0f;
			NewSoundParams.Volume = VolumeMod * VolumeBase * NewRandVolume;

			// Get the new sound's pitch
			float NewRandPitch = ModulationBehavior.bRandomizePitch ? FMath::FRandRange(ModulationBehavior.PitchMin, ModulationBehavior.PitchMax) : 1.0f;
			NewSoundParams.Pitch = PitchMod * PitchBase * NewRandPitch;

			// Get the new sound's fade in time
			float NewRandFadeIn = ModulationBehavior.bFadeVolume ? FMath::FRandRange(ModulationBehavior.MinFadeInTime, ModulationBehavior.MaxFadeInTime) : 0.0f;
			NewSoundParams.FadeInTime = bFirstSpawn ? FMath::Max(FadeInMin, NewRandFadeIn) : NewRandFadeIn;
			NewSoundParams.FadeInTime = (bFirstSpawn && ModulationBehavior.bOnlyFadeInOnRetrigger) ? 0.0f : NewSoundParams.FadeInTime;

			// Get the new sound's start time
			NewSoundParams.StartTime = PlaybackBehavior.bRandomizeStartingSeekTime ? FMath::FRandRange(0.0f, Sound->Duration) : 0.0f;

			FVector ListenerLocation = FVector::ZeroVector;
			FVector ListenerForward = FVector::ZeroVector;
			FVector ListenerUp = FVector::ZeroVector;

			if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
			{
				// Get available Listener Proxies
				TArray<FListenerProxy>& ListenerProxies = AudioDevice->ListenerProxies;

				// TODO: Handle Split Screen
				if (ListenerProxies.Num())
				{
					// Get Location and Direction from Listener
					const FTransform ListenerTransform = ListenerProxies[0].Transform;
					ListenerLocation = ListenerTransform.GetLocation();
					ListenerForward = ListenerTransform.GetRotation().GetForwardVector();
					ListenerUp = ListenerTransform.GetRotation().GetUpVector();
				}
			}

			// Get random distance based on range
			float NewSpawnDistance = FMath::RandRange(SpawnBehavior.MinSpawnDistance, SpawnBehavior.MaxSpawnDistance);

			// Get angle from min/max range then
			float NewSpawnAngle = FMath::RandRange(SpawnBehavior.MinSpawnAngle, SpawnBehavior.MaxSpawnAngle);

			// Set up altitudinal clamp offset
			float AltitudinalClampOffset = 0.0f;

			if (SpawnBehavior.ClampMode == ESoundscapeColorAltitudeClampMode::Relative)
			{
				AltitudinalClampOffset = ListenerLocation.Z;
			}

			// Sometimes Left Sometimes Right
			if (FMath::RandBool())
			{
				NewSpawnAngle = NewSpawnAngle * (-1.0f);
			}

			// Get X Y Vector
			NewSoundParams.Location = ListenerForward.RotateAngleAxis(NewSpawnAngle, ListenerUp);

			// Random Z Vector
			NewSoundParams.Location.Z = FMath::RandRange(-1.0f, 1.0f);

			// Scale Vector and add to Listener Location
			NewSoundParams.Location = (NewSoundParams.Location * NewSpawnDistance) + ListenerLocation;

			// Clamp the Z Altitude
			NewSoundParams.Location.Z = SpawnBehavior.bClampHeight ? FMath::Clamp(NewSoundParams.Location.Z, AltitudinalClampOffset + SpawnBehavior.MinSpawnHeightClamp, AltitudinalClampOffset + SpawnBehavior.MaxSpawnHeightClamp) : NewSoundParams.Location.Z;

			if (SpawnBehavior.bRotateSoundSource)
			{
				NewSoundParams.Rotation.Yaw = FMath::Max(0.0f, FMath::FRandRange(SpawnBehavior.MinAzimuthalRotationAngle, SpawnBehavior.MaxAzimuthalRotationAngle));
				NewSoundParams.Rotation.Pitch = FMath::Max(0.0f, FMath::FRandRange(SpawnBehavior.MinAltitudinalRotationAngle, SpawnBehavior.MaxAltitudinalRotationAngle));
			}
		}
	}
}

bool UActiveSoundscapeColor::EvaluateColorPointDensity()
{
	// Check if we actually want to filter by color point density, otherwise return true
	if (SpawnBehavior.bFilterByColorPointDensity)
	{
		// Get World
		UWorld* World = GetWorld();

		// Return false if failed to get world
		if (!World)
		{
			return false;
		}

		// Get Soundscape Subsystem
		UGameInstance* GameInstance = World->GetGameInstance();

		if (!GameInstance)
		{
			return false;
		}

		// Check to see if Filter By Color Point Density is TRUE, determine if Soundscape Subsystem is still valid
		if (USoundscapeSubsystem* SoundscapeSubsystem = GameInstance->GetSubsystem<USoundscapeSubsystem>())
		{
			// Cache Color Point Density of specified type
			int32 ColorPointDensity = SoundscapeSubsystem->CheckColorPointDensity(NewSoundParams.Location, SpawnBehavior.ColorPoint);

			// If Color Point Density is under threshold, set spawn flag to false
			if (SpawnBehavior.MinColorPointNumber > ColorPointDensity)
			{
				// Attempt Draw Debug Cell
				SoundscapeSubsystem->DrawDebugCell(NewSoundParams.Location, false);

				// Failed to meet Color Point Density requirements, early out and do not spawn sound
				return false;
			}
			
			// Attempt Draw Debug Cell
			SoundscapeSubsystem->DrawDebugCell(NewSoundParams.Location, true);

			// We have met the threshold, return success
			return true;
		}

		// If we failed getting the Game Instance, we do not want to return success
		return false;
	}

	return true;
}

// Spawn a Color Voice
void UActiveSoundscapeColor::SpawnSoundscapeColorVoice()
{
	if (UWorld* World = GetWorld())
	{
		// Play Sound if needed and valid sound available
		if (Sound)
		{
			// Set up Voice and Component
			UActiveSoundscapeColorVoice* NewActiveSoundscapeColorVoice = nullptr;
			UAudioComponent* NewAudioComponent = nullptr;

			// If there's a free voice in the pool, get first available voice
			if (bVoiceInPoolFree)
			{
				for (UActiveSoundscapeColorVoice* ActiveSoundscapeColorVoice : SoundscapeColorVoicePool)
				{
					if (ActiveSoundscapeColorVoice)
					{
						// Get first free voice
						if (ActiveSoundscapeColorVoice->bFree)
						{
							// Cache pointer to this voice
							NewActiveSoundscapeColorVoice = ActiveSoundscapeColorVoice;

							break;
						}

					}
				}
			}

			// If the ActiveSoundscapeColorVoice is still null, create a new one
			if (NewActiveSoundscapeColorVoice == nullptr)
			{
				// Create new voice
				NewActiveSoundscapeColorVoice = NewObject<UActiveSoundscapeColorVoice>(World);

				// Add new voice to the pool
				SoundscapeColorVoicePool.Add(NewActiveSoundscapeColorVoice);
			}

			// If the Audio Component invalid, create a new one
			if (NewActiveSoundscapeColorVoice->AudioComponent == nullptr)
			{
				NewActiveSoundscapeColorVoice->AudioComponent = NewObject<UAudioComponent>(World);
			}

			// Cache Audio Component ptr
			NewAudioComponent = NewActiveSoundscapeColorVoice->AudioComponent;

			// Set World location of Audio Component
			NewAudioComponent->SetWorldLocationAndRotation(NewSoundParams.Location, NewSoundParams.Rotation.Quaternion());

			// Set relevant Audio Component values before playing
			NewAudioComponent->SetPitchMultiplier(NewSoundParams.Pitch);
			NewAudioComponent->SetSound(Sound);

			NewAudioComponent->bAutoDestroy = false;

			// Play the sound
			NewAudioComponent->FadeIn(NewSoundParams.FadeInTime, NewSoundParams.Volume, NewSoundParams.StartTime);

			// Voice is no longer free
			NewActiveSoundscapeColorVoice->bFree = false;

			// Handle limited duration sounds
			if (PlaybackBehavior.bLimitPlaybackDuration)
			{
				float FadeOutDuration = FMath::Max(0.0001f, FMath::FRandRange(ModulationBehavior.MinFadeOutTime, ModulationBehavior.MaxFadeOutTime));
				float LimitedDuration = FMath::Max(0.0001f, FMath::FRandRange(PlaybackBehavior.MinPlaybackDuration, PlaybackBehavior.MaxPlaybackDuration));
				FTimerDelegate TimerDelegate;

				TimerDelegate.BindUFunction(NewActiveSoundscapeColorVoice, FName("StopLimitedDurationVoice"), FadeOutDuration);
				GetWorld()->GetTimerManager().SetTimer(NewActiveSoundscapeColorVoice->TimerHandle, TimerDelegate, LimitedDuration, false);
			}

		}


	}
}
