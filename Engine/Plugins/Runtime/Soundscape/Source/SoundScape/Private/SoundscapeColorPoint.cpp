// Copyright Epic Games, Inc. All Rights Reserved.


#include "SoundscapeColorPoint.h"
#include "SoundscapeSubsystem.h"
#include "Engine/World.h"
#include "Engine/GameInstance.h"

// Sets default values for this component's properties
USoundscapeColorPointComponent::USoundscapeColorPointComponent()
{
	// Set this component to be initialized when the game starts, and to be ticked every frame.  You can turn these features
	// off to improve performance if you don't need them.
	PrimaryComponentTick.bCanEverTick = false;
	// ...
}


void USoundscapeColorPointComponent::GetInfo(FGameplayTag& ColorPointOut, FVector& LocationOut) const
{
	// Get and validate Owner Actor
	if (const AActor* Owner = GetOwner())
	{
		// Set Location and Color Point values
		LocationOut = Owner->GetActorLocation();
		ColorPointOut = ColorPoint;
	}
}

// Called when the game starts
void USoundscapeColorPointComponent::BeginPlay()
{
	// Add this Active Color Point to the Subsystem
	if (const UWorld* World = GetWorld())
	{
		if (const UGameInstance* GameInstance = World->GetGameInstance())
		{
			if (USoundscapeSubsystem* SoundscapeSubsystem = GameInstance->GetSubsystem<USoundscapeSubsystem>())
			{
				SoundscapeSubsystem->AddActiveColorPoint(this);
			}
		}
	}

	Super::BeginPlay();
}

void USoundscapeColorPointComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// Remove this Active Color Point from the Subsystem
	if (const UWorld* World = GetWorld())
	{
		if (const UGameInstance* GameInstance = World->GetGameInstance())
		{
			if (USoundscapeSubsystem* SoundscapeSubsystem = GameInstance->GetSubsystem<USoundscapeSubsystem>())
			{
				SoundscapeSubsystem->RemoveActiveColorPoint(this);
			}
		}
	}

	Super::EndPlay(EndPlayReason);
}

