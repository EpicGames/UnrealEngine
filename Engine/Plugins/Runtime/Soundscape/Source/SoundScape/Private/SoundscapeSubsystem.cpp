// Copyright Epic Games, Inc. All Rights Reserved.


#include "SoundscapeSubsystem.h"
#include "DrawDebugHelpers.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "SoundscapeColorPoint.h"
#include "SoundscapeSettings.h"
#include "SoundScapePalette.h"
#include "AudioDevice.h"
#include "TimerManager.h"
#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"

static FAutoConsoleCommandWithWorld GResetSoundscape(
	TEXT("soundscape.ResetSoundscape"),
	TEXT("Forces restart on all active Soundscape Palettes."),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* World)
{
	if (!World)
	{
		return;
	}

	if (const UGameInstance* GameInstance = World->GetGameInstance())
	{
		if (USoundscapeSubsystem* SoundscapeSubsystem = GameInstance->GetSubsystem<USoundscapeSubsystem>())
		{
			SoundscapeSubsystem->RestartSoundscape();
		}
	}
}));

void USoundscapeColorPointHashMap::InitializeHash(float HashCellSizeIn, FVector GridCenterIn)
{
	// Cache Hash Cell Size
	HashCellSize = FMath::Max(HashCellSizeIn, 1.0f);

	// MaxGridWidth / Hash Cell Size
	GridWidth = FMath::FloorToInt(MaxGridWidth / HashCellSize);

	// Grid Width Cubed
	NumCells = int64(GridWidth) * int64(GridWidth) * int64(GridWidth);

	// Cell Size Inverted
	HashCellFactor = 1 / HashCellSize;

	// Cache Grid Centerpoint
	GridCenter = GridCenterIn;

	// Calculate Grid Origin Offset
	GridOriginOffset = GridCenter - FVector(float(HalfMaxGridWidth), float(HalfMaxGridWidth), float(HalfMaxGridWidth));
}

void USoundscapeColorPointHashMap::ClearHash()
{
	// Cycle through Color Point Hash Map
	for (auto It = ColorPointHashMap.CreateIterator(); It; ++It)
	{
		// If Color Point has an existing hash map, empty it
		if (USoundscapeColorPointHashCellDensity* ColorPointHashCell = It->Value)
		{
			ColorPointHashCell->ColorPointHashCellDensity.Empty();
		}
	}

	// Empty Hash Map
	ColorPointHashMap.Empty();
}

int32 USoundscapeColorPointHashMap::NumColorPointsInCell(const FVector& Location, const FGameplayTag ColorPoint)
{
	// Determine if the Hash Map contains the Color Point Density Table Pointer
	if (TObjectPtr<USoundscapeColorPointHashCellDensity>* ColorPointHashCellPtr = ColorPointHashMap.Find(ColorPoint))
	{
		// Validate the double pointer
		if (USoundscapeColorPointHashCellDensity* ColorPointHashCell = *ColorPointHashCellPtr)
		{
			// Validate the Color Point Density Map
			if (ColorPointHashCell != nullptr)
			{
				// Look up Color Point Density at that Location
				int32* CellDensity = ColorPointHashCell->ColorPointHashCellDensity.Find(CalculateHashIndex(Location));

				// If Cell Density pointer is valid, return value
				if (CellDensity != nullptr)
				{
					return *CellDensity;
				}
			}
		}
	}

	// Otherwise return 0
	return 0;
}

bool USoundscapeColorPointHashMap::AddColorPointToHash(const FVector& Location, const FGameplayTag ColorPoint)
{
	// Add a single Color Point to the Color Point Hash Map
	// If the Color Point Coordinates exceed the Max Grid Width, it is out of bounds of the hash table
	// Incoming Location values are added to the Grid Origin Offset in the case that the Grid is not centered around Origin 0,0,0
	if (FMath::Abs(GridCenter.X + Location.X) > HalfMaxGridWidth
		|| FMath::Abs(GridCenter.Y + Location.Y) > HalfMaxGridWidth
		|| FMath::Abs(GridCenter.Z + Location.Z) > HalfMaxGridWidth)
	{
		UE_LOGF(LogSoundscapeSubsystem, Warning, "Location out of Hash Bounds");

		// Return false, unable to add Color Point to Hash
		return false;
	}

	// Get ColorPoint Density Map
	USoundscapeColorPointHashCellDensity* ColorPointDensityMap = GetColorPointDensityMap(ColorPoint);

	// Create a new Color Point Hash for this Color Point if one needs to be created
	if (ColorPointDensityMap == nullptr)
	{
		// Create new Color Point Density Map
		ColorPointDensityMap = NewObject<USoundscapeColorPointHashCellDensity>(this);

		// Add to Color Point Hash Map
		ColorPointHashMap.Add(ColorPoint, ColorPointDensityMap);
	}


	// If a Color Point Map exists for this Color Point, use it
	if (ColorPointDensityMap)
	{
		// Cache Hash Index
		uint64 HashIndex = CalculateHashIndex(Location);

		// Get potential Density value from Hash Index
		int32* ColorPointDensity = ColorPointDensityMap->ColorPointHashCellDensity.Find(HashIndex);

		// Validate Density pointer
		if (ColorPointDensity != nullptr)
		{
			// Color Point already tracked at this hash cell, increment
			++(*ColorPointDensity);
		}
		else
		{
			// Density Value invalid, new Hash Key/Value must be added at that index
			ColorPointDensityMap->ColorPointHashCellDensity.Add(HashIndex, 1);
		}
	}

	// Color Point successfully added
	return true;
}

void USoundscapeColorPointHashMap::AddColorPointArrayToHash(const TArray<FVector>& Locations, const FGameplayTag ColorPoint)
{

	// Local cache for Hash Indices
	TArray<uint64> HashIndices;

	// Get ColorPoint Density Map
	USoundscapeColorPointHashCellDensity* ColorPointDensityMap = GetColorPointDensityMap(ColorPoint);

	// Create a new Color Point Hash for this Color Point if one needs to be created
	if (ColorPointDensityMap == nullptr)
	{
		// Create new Color Point Density Map
		ColorPointDensityMap = NewObject<USoundscapeColorPointHashCellDensity>(this);

		// Add to Color Point Hash Map
		ColorPointHashMap.Add(ColorPoint, ColorPointDensityMap);
	}


	// If a Color Point Map exists for this Color Point, use it
	if(ColorPointDensityMap)
	{
		// Cycle through input Locations, add to temp array the indices calculated by the hashing function
		for (auto IndividualLocation = Locations.CreateConstIterator(); IndividualLocation; ++IndividualLocation)
		{
			// Make sure incoming Location Vector is within Hash Map Bounds
			if (FMath::Abs(GridCenter.X + IndividualLocation->X) <= HalfMaxGridWidth
				&& FMath::Abs(GridCenter.Y + IndividualLocation->Y) <= HalfMaxGridWidth
				&& FMath::Abs(GridCenter.Z + IndividualLocation->Z) <= HalfMaxGridWidth)
			{
				// Add Hash Index to temp Array
				HashIndices.Add(CalculateHashIndex(*IndividualLocation));
			}
			else
			{
				// If Location out of Hash Map Bounds, throw it away and spit out a warning
				UE_LOGF(LogSoundscapeSubsystem, Warning, "Location out of Hash Bounds");
			}
		}

		// Cycle through temp array of hash indices
		for (auto HashIndex = HashIndices.CreateConstIterator(); HashIndex; ++HashIndex)
		{
			// Cache index
			uint64 NewKey = *HashIndex;

			// Look for Color Point Density at that Index
			int32* ColorPointDensity = ColorPointDensityMap->ColorPointHashCellDensity.Find(NewKey);

			// If Color Point Density is valid, 
			if (ColorPointDensity != nullptr)
			{
				// Increment the value at the pointer
				++(*ColorPointDensity);
			}
			else
			{
				// Add a new Cell
				ColorPointDensityMap->ColorPointHashCellDensity.Add(NewKey, 1);
			}
		}

	}
}

uint64 USoundscapeColorPointHashMap::CalculateHashIndex(const FVector& Location)
{
	const FVector OffsetLocation = Location - GridOriginOffset;
	uint64 HashX = uint64(FMath::FloorToInt(OffsetLocation.X * HashCellFactor));
	uint64 HashY = uint64(FMath::FloorToInt(OffsetLocation.Y * HashCellFactor)) * uint64(GridWidth);
	uint64 HashZ = uint64(FMath::FloorToInt(OffsetLocation.Z * HashCellFactor)) * (uint64(GridWidth) * uint64(GridWidth));

	return HashX + HashY + HashZ;
}

void USoundscapeColorPointHashMap::SetGridCenterpoint(const FVector& Location)
{
	// Cache new Centerpoint
	GridCenter = Location;

	// Calculate Grid Origin Offset
	GridOriginOffset = Location - FVector(float(HalfMaxGridWidth), float(HalfMaxGridWidth), float(HalfMaxGridWidth));
}

TMap<FGameplayTag, int32> USoundscapeColorPointHashMap::GetHashMapColorPointDensitySummary()
{
	TMap<FGameplayTag, int32> DensitySummary;

	for (auto It = ColorPointHashMap.CreateConstIterator(); It; ++It)
	{
		FGameplayTag ColorPoint = It.Key();

		if (ColorPoint.IsValid())
		{
			int32 ColorDensity = 0;

			if (USoundscapeColorPointHashCellDensity* ColorPointHashLayer = It.Value())
			{
				for (auto Jt = ColorPointHashLayer->ColorPointHashCellDensity.CreateConstIterator(); Jt; ++Jt)
				{
					ColorDensity += Jt.Value();
				}
			}

			DensitySummary.Add(ColorPoint, ColorDensity);
		}
	}

	return DensitySummary;
}

USoundscapeColorPointHashCellDensity* USoundscapeColorPointHashMap::GetColorPointDensityMap(FGameplayTag ColorPoint)
{
	if (TObjectPtr<USoundscapeColorPointHashCellDensity>* ColorPointHashCellPtr = ColorPointHashMap.Find(ColorPoint))
	{
		// Get Color Point Hash Cell Pointer
		return *ColorPointHashCellPtr;
	}

	return nullptr;
}

const FVector USoundscapeColorPointHashMap::GetHashCellCenterpoint(FVector Location)
{
	const int32 LocationX = FMath::FloorToInt((Location.X - GridOriginOffset.X) / HashCellSize);
	const int32 LocationY = FMath::FloorToInt((Location.Y - GridOriginOffset.Y) / HashCellSize);
	const int32 LocationZ = FMath::FloorToInt((Location.Z - GridOriginOffset.Z) / HashCellSize);

	const float HalfHashCellSize = HashCellSize * 0.5;

	return FVector(((float)LocationX * HashCellSize) + HalfHashCellSize + GridOriginOffset.X
				, ((float)LocationY * HashCellSize) + HalfHashCellSize + GridOriginOffset.Y
				, ((float)LocationZ * HashCellSize) + HalfHashCellSize + GridOriginOffset.Z);
}

const float USoundscapeColorPointHashMap::GetHashCellWidth()
{
	return HashCellSize;
}

void USoundscapeColorPointHashMapCollection::InitializeCollection()
{
	// Get plugin settings on Subsystem initialization
	if (const USoundscapeSettings* ProjectSettings = GetDefault<USoundscapeSettings>())
	{
		// Hash Cell Width for LOD1
		LOD1ColorPointHashWidth = FMath::Max(ProjectSettings->LOD1ColorPointHashWidth, 1.0f);

		// Hash Cell LOD1 Max Distance
		LOD1ColorPointHashDistance = FMath::Max(ProjectSettings->LOD1ColorPointHashDistance, 1.0f);

		// Hash Cell Width for LOD2
		LOD2ColorPointHashWidth = FMath::Max(ProjectSettings->LOD2ColorPointHashWidth, 1.0f);

		// Hash Cell LOD2 Max Distance
		LOD2ColorPointHashDistance = FMath::Max(ProjectSettings->LOD2ColorPointHashDistance, 1.0f);

		// Hash Cell Width for LOD3
		LOD3ColorPointHashWidth = FMath::Max(ProjectSettings->LOD3ColorPointHashWidth, 1.0f);
	}

	// Initialize LOD Hashes
	ColorPointHashMapLOD1 = NewObject<USoundscapeColorPointHashMap>(this);
	ColorPointHashMapLOD1->InitializeHash(LOD1ColorPointHashWidth, FVector::ZeroVector);

	ColorPointHashMapLOD2 = NewObject<USoundscapeColorPointHashMap>(this);
	ColorPointHashMapLOD2->InitializeHash(LOD2ColorPointHashWidth, FVector::ZeroVector);

	ColorPointHashMapLOD3 = NewObject<USoundscapeColorPointHashMap>(this);
	ColorPointHashMapLOD3->InitializeHash(LOD3ColorPointHashWidth, FVector::ZeroVector);
}

void USoundscapeColorPointHashMapCollection::AddColorPointArrayToHashMapCollection(const TArray<FVector>& Locations, const FGameplayTag ColorPoint)
{
	if (ColorPointHashMapLOD1)
	{
		ColorPointHashMapLOD1->AddColorPointArrayToHash(Locations, ColorPoint);
	}

	if (ColorPointHashMapLOD2)
	{
		ColorPointHashMapLOD2->AddColorPointArrayToHash(Locations, ColorPoint);
	}

	if (ColorPointHashMapLOD3)
	{
		ColorPointHashMapLOD3->AddColorPointArrayToHash(Locations, ColorPoint);
	}
}

void USoundscapeColorPointHashMapCollection::ClearColorPointHashMapCollection()
{
	if (ColorPointHashMapLOD1)
	{
		ColorPointHashMapLOD1->ClearHash();
	}

	if (ColorPointHashMapLOD2)
	{
		ColorPointHashMapLOD2->ClearHash();
	}

	if (ColorPointHashMapLOD3)
	{
		ColorPointHashMapLOD3->ClearHash();
	}
}

int32 USoundscapeColorPointHashMapCollection::GetColorPointHashMapCollectionDensity(const FVector Location, const FGameplayTag ColorPoint, const ESoundscapeLOD SoundscapeLOD)
{
	if (ColorPointHashMapLOD1 && ColorPointHashMapLOD2 && ColorPointHashMapLOD3)
	{
		switch (SoundscapeLOD)
		{
		case ESoundscapeLOD::LOD1:
			return ColorPointHashMapLOD1->NumColorPointsInCell(Location, ColorPoint);

		case ESoundscapeLOD::LOD2:
			return ColorPointHashMapLOD2->NumColorPointsInCell(Location, ColorPoint);

		case ESoundscapeLOD::LOD3:
			return ColorPointHashMapLOD3->NumColorPointsInCell(Location, ColorPoint);
		}
	}

	return 0;
}

void USoundscapeColorPointHashMapCollection::CalculateTotalColorPointDensity(TMap<FGameplayTag, int32>& TotalHashMapColorPointDensity, ESoundscapeLOD SoundscapeLOD)
{
	if (ColorPointHashMapLOD1 && ColorPointHashMapLOD2 && ColorPointHashMapLOD3)
	{
		switch (SoundscapeLOD)
		{
		case ESoundscapeLOD::LOD1:
			TotalHashMapColorPointDensity = ColorPointHashMapLOD1->GetHashMapColorPointDensitySummary();

		case ESoundscapeLOD::LOD2:
			TotalHashMapColorPointDensity = ColorPointHashMapLOD2->GetHashMapColorPointDensitySummary();

		case ESoundscapeLOD::LOD3:
			TotalHashMapColorPointDensity = ColorPointHashMapLOD3->GetHashMapColorPointDensitySummary();
		}
	}
}

void USoundscapeSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UE_LOGF(LogSoundscapeSubsystem, Verbose, "Subsystem Initializing");

	// Get Subsystem World
	UWorld* World = GetWorld();

	// Initialize AudioDeviceID
	AudioDeviceID = INDEX_NONE;

	// Get world
	if (World)
	{
		// Get Audio Device Handle
		FAudioDevice* AudioDevice = World->GetAudioDeviceRaw();

		if (AudioDevice)
		{
			AudioDeviceID = AudioDevice->DeviceID;
		}

	}

	// Get plugin settings on Subsystem initialization
	if (const USoundscapeSettings* ProjectSettings = GetDefault<USoundscapeSettings>())
	{
		if (ProjectSettings->SoundscapePaletteCollection.IsEmpty() == false)
		{
			FSoundscapePaletteCollection PaletteCollectionToAdd;

			PaletteCollectionToAdd.SoundscapePaletteCollection = ProjectSettings->SoundscapePaletteCollection;

			UE_LOGF(LogSoundscapeSubsystem, Verbose, "Project settings Soundscape Palette Collection is valid");

			AddPaletteCollection(FName("GlobalPaletteCollection"), PaletteCollectionToAdd);			
		}
		else
		{
			UE_LOGF(LogSoundscapeSubsystem, Verbose, "Project settings Soundscape Palette Collection is empty");
		}

		bDebugMode = ProjectSettings->bDebugDraw;

		// Hash Cell Width for LOD1
		LOD1ColorPointHashWidth = FMath::Max(ProjectSettings->LOD1ColorPointHashWidth, 1.0f);

		// Hash Cell LOD1 Max Distance
		LOD1ColorPointHashDistance = FMath::Max(ProjectSettings->LOD1ColorPointHashDistance, 1.0f);

		// Hash Cell Width for LOD2
		LOD2ColorPointHashWidth = FMath::Max(ProjectSettings->LOD2ColorPointHashWidth, 1.0f);

		// Hash Cell LOD2 Max Distance
		LOD2ColorPointHashDistance = FMath::Max(ProjectSettings->LOD2ColorPointHashDistance, 1.0f);

		// Hash Cell Width for LOD3
		LOD3ColorPointHashWidth = FMath::Max(ProjectSettings->LOD3ColorPointHashWidth, 1.0f);

		// Hash Cell Width for the Active Hash
		ActiveColorPointHashWidth = FMath::Max(ProjectSettings->ActiveColorPointHashWidth, 1.0f);

		// Hash Cell Update Timing for the Active Hash
		ActiveColorPointHashUpdateTimeSeconds = FMath::Max(ProjectSettings->ActiveColorPointHashUpdateTimeSeconds, 0.2f);
	}
	else
	{
		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Invalid Project Settings");
	}

	// Initialize LOD Hashes
	ColorPointHashMapLOD1 = NewObject<USoundscapeColorPointHashMap>(this);
	ColorPointHashMapLOD1->InitializeHash(LOD1ColorPointHashWidth, FVector::ZeroVector);

	ColorPointHashMapLOD2 = NewObject<USoundscapeColorPointHashMap>(this);
	ColorPointHashMapLOD2->InitializeHash(LOD2ColorPointHashWidth, FVector::ZeroVector);

	ColorPointHashMapLOD3 = NewObject<USoundscapeColorPointHashMap>(this);
	ColorPointHashMapLOD3->InitializeHash(LOD3ColorPointHashWidth, FVector::ZeroVector);

	// Initialize Active Hash Map
	ActiveColorPointHashMap = NewObject<USoundscapeColorPointHashMap>(this);
	ActiveColorPointHashMap->InitializeHash(ActiveColorPointHashWidth, FVector::ZeroVector);

	UpdateActiveColorPointHashMap();
}

void USoundscapeSubsystem::Deinitialize()
{
	UE_LOGF(LogSoundscapeSubsystem, Verbose, "Subsystem Deinitializing");

	ActiveColorPointUpdateTimer.Invalidate();

	for(const TPair<FName, TSharedPtr<FStreamableHandle>>& HandleSharedPtr : ActiveAssetLoadHandles)
	{
		if(HandleSharedPtr.Value.IsValid())
		{
			if (FStreamableHandle* HandlePtr = HandleSharedPtr.Value.Get())
			{
				HandlePtr->CancelHandle();
			}
		}
	}
}

bool USoundscapeSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	// Check with parent first
	if (Super::ShouldCreateSubsystem(Outer) == false)
	{
		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Subsystem will not be created based on parent rules");

		return false;
	}

	// Get World
	if (UWorld* OuterWorld = Outer->GetWorld())
	{
		if (FAudioDevice* OuterAudioDevice = OuterWorld->GetAudioDeviceRaw())
		{
			UE_LOGF(LogSoundscapeSubsystem, Verbose, "Audio device valid, will create subsystem");

			// If we do have an audio device, we can create this subsystem.
			return true;
		}
	}

	UE_LOGF(LogSoundscapeSubsystem, Verbose, "Subsystem will not be created");

	// If we do not have an audio device, we do not need to create this subsystem.
	return false;
}

void USoundscapeSubsystem::SetState(FGameplayTag SoundscapeState)
{
	if (SoundscapeState.IsValid())
	{
		// Add new state to the container
		SubsystemState.AddLeafTag(SoundscapeState);

		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Set State called");

		UpdateState();
	}
	else
	{
		UE_LOGF(LogSoundscapeSubsystem, Warning, "Set State called with invalid State Tag");
	}
}

void USoundscapeSubsystem::ClearState(FGameplayTag SoundscapeState)
{
	if (SoundscapeState.IsValid())
	{
		// Add new state to the container
		SubsystemState.RemoveTag(SoundscapeState);

		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Clear State called");

		UpdateState();
	}
	else
	{
		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Clear State called with invalid State Tag");
	}
}

void USoundscapeSubsystem::RestartSoundscape()
{
	// Restart matching valid Soundscape Palettes
	for (const TPair<USoundscapePalette*, UActiveSoundscapePalette*> ActiveSoundscapePalettePair : ActivePalettes)
	{
		USoundscapePalette* SoundscapePalette = ActiveSoundscapePalettePair.Key;
		UActiveSoundscapePalette* ActiveSoundscapePalette = ActiveSoundscapePalettePair.Value;

		if (SoundscapePalette && ActiveSoundscapePalette)
		{
			ActiveSoundscapePalette->Stop();
			ActiveSoundscapePalette->Play();
		}
	}
}

bool USoundscapeSubsystem::AddPaletteCollection(FName PaletteCollectionName, FSoundscapePaletteCollection PaletteCollection)
{
	if (UnloadedPaletteCollections.Contains(PaletteCollectionName) == false)
	{
		UnloadedPaletteCollections.Add(PaletteCollectionName, PaletteCollection);

		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Palette Collection %ls added", *PaletteCollectionName.ToString());

		return LoadPaletteCollection(PaletteCollectionName);
	}

	UE_LOGF(LogSoundscapeSubsystem, Verbose, "Add Palette Collection called with empty Palette Collection");

	return false;
}

bool USoundscapeSubsystem::RemovePaletteCollection(FName PaletteCollectionName)
{
	if (UnloadedPaletteCollections.Find(PaletteCollectionName))
	{
		if (UnloadPaletteCollection(PaletteCollectionName))
		{
			// Release the handle pointer if Palette Collection still loading
			if (TSharedPtr<FStreamableHandle>* AssetHandlePtr = ActiveAssetLoadHandles.Find(PaletteCollectionName))
			{
				TSharedPtr<FStreamableHandle> AssetHandle = *AssetHandlePtr;

				if (AssetHandle.IsValid())
				{
					AssetHandle.Get()->CancelHandle();
					AssetHandle.Reset();
				}

				ActiveAssetLoadHandles.Remove(PaletteCollectionName);
			}

			// Unloaded, now remove
			UnloadedPaletteCollections.Remove(PaletteCollectionName);

			UE_LOGF(LogSoundscapeSubsystem, Verbose, "Palette Collection %ls removed", *PaletteCollectionName.ToString());

			UpdateState();

			return true;
		}
	}

	UE_LOGF(LogSoundscapeSubsystem, Verbose, "Remove Palette Collection called, but the name could not be found");

	return false;
}

bool USoundscapeSubsystem::LoadPaletteCollection(FName PaletteCollectionName)
{
	if (UnloadedPaletteCollections.Contains(PaletteCollectionName))
	{
		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Attempting to load Palette Collection %ls", *PaletteCollectionName.ToString());

		FSoundscapePaletteCollection PaletteCollectionToLoad = UnloadedPaletteCollections.FindRef(PaletteCollectionName);

		// Early out as false if Collection Empty
		if (PaletteCollectionToLoad.SoundscapePaletteCollection.Num() <= 0)
		{
			UE_LOGF(LogSoundscapeSubsystem, Verbose, "Attempted to load Palette Collection %ls but Collection Empty", *PaletteCollectionName.ToString());

			return false;
		}


		// Set up Asset Path Array
		TArray<FSoftObjectPath> AssetsToLoad;
		AssetsToLoad.Reserve(PaletteCollectionToLoad.SoundscapePaletteCollection.Num());

		// Copy paths over to array
		for (FSoftObjectPath AssetPath : PaletteCollectionToLoad.SoundscapePaletteCollection)
		{
			AssetsToLoad.Add(AssetPath);
		}

		// Create a callback delegate for completing the async load request
		FStreamableDelegate AssetLoadingDelegate = FStreamableDelegate::CreateUObject(this, &USoundscapeSubsystem::CompleteAsyncLoading, PaletteCollectionName);

		// Set up shared pointer for Asset Manager Async Load Request
		TSharedPtr<FStreamableHandle> AssetHandle;
		AssetHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(MoveTemp(AssetsToLoad), MoveTemp(AssetLoadingDelegate));

		// Add Shared Pointer to map, allowing multiple load calls in a single frame, if necessary
		ActiveAssetLoadHandles.Add(PaletteCollectionName, AssetHandle);

		return true;
	}

	UE_LOGF(LogSoundscapeSubsystem, Verbose, "Attempted to load Palette Collection %ls but failed", *PaletteCollectionName.ToString());

	return false;
}

void USoundscapeSubsystem::CompleteAsyncLoading(FName PaletteCollectionName)
{
	FSoundscapePaletteCollection PaletteCollectionToLoad = UnloadedPaletteCollections.FindRef(PaletteCollectionName);

	for (FSoftObjectPath& ObjPath : PaletteCollectionToLoad.SoundscapePaletteCollection)
	{
		if (UObject* Object = ObjPath.ResolveObject())
		{
			USoundscapePalette* SoundscapePalette = Cast<USoundscapePalette>(Object);

			if (SoundscapePalette)
			{
				// If palette is valid, add it to the Subsystem Collection
				LoadedPaletteCollectionSet.Add(SoundscapePalette);

				UE_LOGF(LogSoundscapeSubsystem, Verbose, "Valid Soundscape Palette %ls in Palette Collection %ls", *SoundscapePalette->GetFullName(), *PaletteCollectionName.ToString());
			}
			else
			{
				UE_LOGF(LogSoundscapeSubsystem, Verbose, "Invalid Soundscape Palette in Palette Collection %ls", *PaletteCollectionName.ToString());
			}
		}
		else
		{
			UE_LOGF(LogSoundscapeSubsystem, Verbose, "Failed to load %ls", *ObjPath.ToString());
		}
	}

	// Release the handle pointer
	if (TSharedPtr<FStreamableHandle>* AssetHandlePtr = ActiveAssetLoadHandles.Find(PaletteCollectionName))
	{
		TSharedPtr<FStreamableHandle> AssetHandle = *AssetHandlePtr;

		if (AssetHandle.IsValid())
		{
			AssetHandle.Reset();
		}

		ActiveAssetLoadHandles.Remove(PaletteCollectionName);
	}

	// Now that we have completed loading Palettes, we will call update to check whether new palettes need to be activated.
	UpdateState();
}

bool USoundscapeSubsystem::UnloadPaletteCollection(FName PaletteCollectionName)
{
	if (UnloadedPaletteCollections.Find(PaletteCollectionName))
	{
		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Attempting to unload Palette Collection %ls", *PaletteCollectionName.ToString());

		// Get unloaded set to remove from list
		FSoundscapePaletteCollection PaletteCollectionToUnload = UnloadedPaletteCollections.FindRef(PaletteCollectionName);
		TSet<FSoftObjectPath> PaletteCollectionSetToUnload;
		PaletteCollectionSetToUnload.Append(PaletteCollectionToUnload.SoundscapePaletteCollection);
		TSet<FSoftObjectPath> PaletteCollectionSetToKeep;

		// Go through all unloaded collections and construct a unique list of Soft Object Paths
		for (TPair<FName, FSoundscapePaletteCollection>& UnloadedCollectionName : UnloadedPaletteCollections)
		{
			if (UnloadedCollectionName.Key != PaletteCollectionName)
			{
				PaletteCollectionSetToKeep.Append(UnloadedCollectionName.Value.SoundscapePaletteCollection.Intersect(PaletteCollectionSetToUnload));
			}
		}

		// Find the difference between what we want to retain and what we want to unload
		PaletteCollectionSetToUnload = PaletteCollectionSetToUnload.Difference(PaletteCollectionSetToKeep);

		// Temp set of palettes to remove and stop
		decltype(LoadedPaletteCollectionSet) PalettesToUnloadStopAndRemove;

		// Go through collection, get their loaded pointers, add to temp removal set
		for (FSoftObjectPath& ObjPath : PaletteCollectionSetToUnload)
		{
			if (UObject* PalettePath = ObjPath.TryLoad())
			{
				USoundscapePalette* SoundscapePalette = Cast<USoundscapePalette>(PalettePath);

				if (SoundscapePalette)
				{
					// If palette is valid, add it to the Subsystem Collection
					PalettesToUnloadStopAndRemove.Add(SoundscapePalette);
				}
			}
		}

		// Stop and remove from Active list
		for (USoundscapePalette* PaletteKeyToStop : PalettesToUnloadStopAndRemove)
		{
			if (PaletteKeyToStop)
			{
				if (UActiveSoundscapePalette* ActiveSoundscapePalette = ActivePalettes.FindRef(PaletteKeyToStop))
				{
					ActiveSoundscapePalette->Stop();

					ActivePalettes.Remove(PaletteKeyToStop);

					UE_LOGF(LogSoundscapeSubsystem, Verbose, "Soundscape Palette %ls stopped", *ActiveSoundscapePalette->GetFullName());
				}
			}
		}

		// Remove from main list of loaded palettes
		LoadedPaletteCollectionSet = LoadedPaletteCollectionSet.Difference(PalettesToUnloadStopAndRemove);

		UE_LOGF(LogSoundscapeSubsystem, Verbose, "Palette Collection %ls unloaded", *PaletteCollectionName.ToString());

		return true;
	}

	UE_LOGF(LogSoundscapeSubsystem, Verbose, "Attempted to unload Palette Collection %ls but could not find matching collection", *PaletteCollectionName.ToString());

	return false;
}

void USoundscapeSubsystem::AddColorPointCollection(FName ColorPointCollectionName, FSoundscapeColorPointCollection ColorPointCollection)
{
	ColorPointCollections.Add(ColorPointCollectionName, ColorPointCollection);

	UpdateColorPointHashMap(*ColorPointHashMapLOD1);
	UpdateColorPointHashMap(*ColorPointHashMapLOD2);
	UpdateColorPointHashMap(*ColorPointHashMapLOD3);
}

bool USoundscapeSubsystem::RemoveColorPointCollection(FName ColorPointCollectionName)
{
	return (ColorPointCollections.Remove(ColorPointCollectionName) > 0);
}

int32 USoundscapeSubsystem::CheckColorPointDensity(FVector Location, FGameplayTag ColorPoint)
{
	FVector ListenerLocation;
	int32 ColorPointDensity = 0;
	ESoundscapeLOD CurrentLOD = ESoundscapeLOD::LOD1;

	if (UWorld* World = GetWorld())
	{
		if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
		{
			// Get available Listener Proxies
			TArray<FListenerProxy>& ListenerProxies = AudioDevice->ListenerProxies;

			// TODO: Handle Split Screen
			if (ListenerProxies.Num())
			{
				// Get Location and Direction from Listener
				FTransform& ListenerTransform = ListenerProxies[0].Transform;
				ListenerLocation = ListenerTransform.GetLocation();
			}
		}
	}

	// Get relative distance from the Listener Location
	FVector RelativeLocation = Location - ListenerLocation;

	// If relative distance is less than LOD 1 cutoff distance, use LOD 1 Map
	if (RelativeLocation.Length() < LOD1ColorPointHashDistance)
	{
		CurrentLOD = ESoundscapeLOD::LOD1;
		ColorPointDensity = ColorPointDensity + ColorPointHashMapLOD1->NumColorPointsInCell(Location, ColorPoint);
	}
	else if (RelativeLocation.Length() < LOD2ColorPointHashDistance)
	{
		CurrentLOD = ESoundscapeLOD::LOD2;
		// If relative distance is less than LOD 2 cutoff distance, use LOD 2 Map
		ColorPointDensity = ColorPointDensity + ColorPointHashMapLOD2->NumColorPointsInCell(Location, ColorPoint);
	}
	else
	{
		CurrentLOD = ESoundscapeLOD::LOD3;
		// If distance exceeds both cutoff distances, use LOD 3 Map
		ColorPointDensity = ColorPointDensity + ColorPointHashMapLOD3->NumColorPointsInCell(Location, ColorPoint);
	}

	// Check Active Color Point Map
	ColorPointDensity = ColorPointDensity + ActiveColorPointHashMap->NumColorPointsInCell(Location, ColorPoint);

	// Check Color Point Hash Map Collections
	if (ColorPointHashMapCollections.Num() > 0)
	{
		for (auto It = ColorPointHashMapCollections.CreateConstIterator(); It; ++It)
		{
			if (It)
			{
				USoundscapeColorPointHashMapCollection* HashMapCollection = *It;

				if (HashMapCollection)
				{
					ColorPointDensity = ColorPointDensity + HashMapCollection->GetColorPointHashMapCollectionDensity(Location, ColorPoint, CurrentLOD);
				}
			}
		}
	}

	// Return total
	return ColorPointDensity;
}

void USoundscapeSubsystem::AddColorPointHashMapCollection(USoundscapeColorPointHashMapCollection* ColorPointHashMapCollection)
{
	if (ColorPointHashMapCollection)
	{
		ColorPointHashMapCollections.Add(ColorPointHashMapCollection);
	}
}

bool USoundscapeSubsystem::RemoveColorPointHashMapCollection(USoundscapeColorPointHashMapCollection* ColorPointHashMapCollection)
{
	if (ColorPointHashMapCollection)
	{
		ColorPointHashMapCollections.RemoveSingleSwap(ColorPointHashMapCollection, EAllowShrinking::Yes);

		return true;
	}

	return false;
}

void USoundscapeSubsystem::DrawDebugCell(FVector Location, bool bSuccess)
{
	if (bDebugMode)
	{
		FVector ListenerLocation;
		TPair<FVector, FVector> BoxDimensions;

		if (UWorld* World = GetWorld())
		{
			if (FAudioDevice* AudioDevice = World->GetAudioDeviceRaw())
			{
				// Get available Listener Proxies
				TArray<FListenerProxy>& ListenerProxies = AudioDevice->ListenerProxies;

				// TODO: Handle Split Screen
				if (ListenerProxies.Num())
				{
					// Get Location and Direction from Listener
					FTransform& ListenerTransform = ListenerProxies[0].Transform;
					ListenerLocation = ListenerTransform.GetLocation();
				}
			}
		}

		// Get relative distance from the Listener Location
		FVector RelativeLocation = Location - ListenerLocation;

		// If relative distance is less than LOD 1 cutoff distance, use LOD 1 Map
		if (RelativeLocation.Length() < LOD1ColorPointHashDistance)
		{
			BoxDimensions = CalculateDebugCellDimensions(Location, ESoundscapeLOD::LOD1);
		}
		else if (RelativeLocation.Length() < LOD2ColorPointHashDistance)
		{
			BoxDimensions = CalculateDebugCellDimensions(Location, ESoundscapeLOD::LOD2);
		}
		else
		{
			BoxDimensions = CalculateDebugCellDimensions(Location, ESoundscapeLOD::LOD3);
		}

		FColor BoxColor = bSuccess ? FColor::Green : FColor::Red;
		DrawDebugBox(GetWorld(), BoxDimensions.Key, BoxDimensions.Value * 0.5f, BoxColor, false, 5.0f, '\000', 5.0f);
	}
}

TPair<FVector, FVector> USoundscapeSubsystem::CalculateDebugCellDimensions(FVector Location, ESoundscapeLOD SoundscapeLOD)
{
	USoundscapeColorPointHashMap* LODHashMap = nullptr;
	TPair<FVector, FVector> BoxResults = { FVector::ZeroVector, FVector::ZeroVector };

	// If relative distance is less than LOD 1 cutoff distance, use LOD 1 Map
	switch (SoundscapeLOD)
	{
	case ESoundscapeLOD::LOD1:
		LODHashMap = ColorPointHashMapLOD1;
		break;
	case ESoundscapeLOD::LOD2:
		LODHashMap = ColorPointHashMapLOD2;
		break;
	case ESoundscapeLOD::LOD3:
		LODHashMap = ColorPointHashMapLOD3;
	}

	if (LODHashMap)
	{
		FVector BoxCenter = LODHashMap->GetHashCellCenterpoint(Location);
		float BoxWidth = LODHashMap->GetHashCellWidth();

		BoxResults.Key = BoxCenter;
		BoxResults.Value = FVector(BoxWidth, BoxWidth, BoxWidth);
	}

	return BoxResults;
}

void USoundscapeSubsystem::AddActiveColorPoint(const USoundscapeColorPointComponent* SoundscapeColorPointComponent)
{
	if (SoundscapeColorPointComponent)
	{
		ActiveSoundscapeColorPointComponents.AddUnique(SoundscapeColorPointComponent);
	}
}

void USoundscapeSubsystem::RemoveActiveColorPoint(const USoundscapeColorPointComponent* SoundscapeColorPointComponent)
{
	if (SoundscapeColorPointComponent)
	{
		ActiveSoundscapeColorPointComponents.RemoveSingleSwap(SoundscapeColorPointComponent, EAllowShrinking::Yes);
	}
}

void USoundscapeSubsystem::UpdateState()
{
	// Palettes the keep
	decltype(ActivePalettes) ActivePalettesToKeep;

	// Remove tags that don't match
	for (const TPair<USoundscapePalette*, UActiveSoundscapePalette*> ActiveSoundscapePalettePair : ActivePalettes)
	{
		USoundscapePalette* SoundscapePalette = ActiveSoundscapePalettePair.Key;
		UActiveSoundscapePalette* ActiveSoundscapePalette = ActiveSoundscapePalettePair.Value;

		if (SoundscapePalette && ActiveSoundscapePalette)
		{
			if (SoundscapePalette->SoundscapePalettePlaybackConditions.IsEmpty() || SoundscapePalette->SoundscapePalettePlaybackConditions.Matches(SubsystemState))
			{
				ActivePalettesToKeep.Add(SoundscapePalette, ActiveSoundscapePalette);
			}
			else
			{
				ActiveSoundscapePalette->Stop();

				UE_LOGF(LogSoundscapeSubsystem, Verbose, "Soundscape Palette %ls stopped", *ActiveSoundscapePalette->GetFullName());
			}
		}
	}

	// Empty active palettes an append ones to keep
	ActivePalettes.Empty();
	ActivePalettes.Append(ActivePalettesToKeep);

	// Add matching palettes
	for (auto PaletteIterator = LoadedPaletteCollectionSet.CreateConstIterator(); PaletteIterator; ++PaletteIterator)
	{
		if (USoundscapePalette* SoundscapePalette = *PaletteIterator)
		{
			// Evaluate if the Soundscape Palette matches
			if ((SoundscapePalette->SoundscapePalettePlaybackConditions.IsEmpty() || SoundscapePalette->SoundscapePalettePlaybackConditions.Matches(SubsystemState)) && ActivePalettes.Contains(SoundscapePalette) == false)
			{
				UWorld* World = GetWorld();

				if (World)
				{
					// Create a new ActiveSoundscapeColor
					UActiveSoundscapePalette* ActiveSoundscapePalette = NewObject<UActiveSoundscapePalette>(World);

					//
					ActiveSoundscapePalette->InitializeSettings(World, SoundscapePalette);

					ActiveSoundscapePalette->Play();

					UE_LOGF(LogSoundscapeSubsystem, Verbose, "Soundscape Palette %ls started", *ActiveSoundscapePalette->GetFullName());

					ActivePalettes.Add(SoundscapePalette, ActiveSoundscapePalette);

				}
			}
		}
	}
}

void USoundscapeSubsystem::UpdateColorPointHashMap(USoundscapeColorPointHashMap& SoundscapeColorPointHashMap)
{
	// Clear out hash Map
	SoundscapeColorPointHashMap.ClearHash();

	// Temp Collection Array
	TArray<FSoundscapeColorPointCollection> ColorPointCollectionArray;

	// Fill Temp Array
	ColorPointCollections.GenerateValueArray(ColorPointCollectionArray);
	
	// Cycle through Collections
	for (auto Collection = ColorPointCollectionArray.CreateConstIterator(); Collection; ++Collection)
	{
		// Cycle through Color Point Vector Arrays
		for (auto ColorPointVectorArray = Collection->ColorPointCollection.CreateConstIterator(); ColorPointVectorArray; ++ColorPointVectorArray)
		{
			// Add Vector Array to Color Point Map
			SoundscapeColorPointHashMap.AddColorPointArrayToHash(ColorPointVectorArray->Locations, ColorPointVectorArray->ColorPoint);
		}
	}
}

void USoundscapeSubsystem::UpdateActiveColorPointHashMap()
{
	ActiveColorPointHashMap->ClearHash();

	for (auto It = ActiveSoundscapeColorPointComponents.CreateConstIterator(); It; ++It)
	{
		if (It)
		{
			if (const USoundscapeColorPointComponent* ActiveColorPointComponent = *It)
			{
				FVector ColorPointLocation;
				FGameplayTag ColorPointValue;

				ActiveColorPointComponent->GetInfo(ColorPointValue, ColorPointLocation);

				ActiveColorPointHashMap->AddColorPointToHash(ColorPointLocation, ColorPointValue);
			}
		}
	}

	// Set up for first Timer delay
	float DelayTime = FMath::Max(0.2f, ActiveColorPointHashUpdateTimeSeconds);

	if (UWorld* World = GetWorld())
	{

		World->GetTimerManager().SetTimer(ActiveColorPointUpdateTimer, this, &USoundscapeSubsystem::UpdateActiveColorPointHashMap, DelayTime);
	}
}

DEFINE_LOG_CATEGORY(LogSoundscapeSubsystem);

