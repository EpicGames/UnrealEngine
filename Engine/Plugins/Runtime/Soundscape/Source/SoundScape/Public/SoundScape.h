// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SoundScapeModule.h"
#include "SoundscapeColor.h"
#include "SoundScapePalette.h"
#include "SoundScape.generated.h"

UCLASS()
class SOUNDSCAPE_API USoundscapeBPFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/** Spawns, but does not play, a new Soundscape Elemental Agent. Returns true if spawn was successful. "Play" can be called on the resulting Active Soundscape Color*/
	UFUNCTION(BlueprintCallable, Category = "Soundscape", meta = (WorldContext = "WorldContextObject"))
	static bool SpawnSoundscapeColor(UObject* WorldContextObject, class USoundscapeColor* SoundscapeColorIn, UActiveSoundscapeColor*& ActiveSoundscapeColor);

	/** Spawns a new Soundscape Palette Agent. Returns true if spawn was successful. */
	UFUNCTION(BlueprintCallable, Category = "Soundscape", meta = (WorldContext = "WorldContextObject"))
	static bool SpawnSoundscapePalette(UObject* WorldContextObject, class USoundscapePalette* SoundscapePaletteIn, UActiveSoundscapePalette*& SoundscapePaletteAgentsOut);

};
