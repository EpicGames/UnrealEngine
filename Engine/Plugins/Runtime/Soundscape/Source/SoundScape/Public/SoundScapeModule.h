// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSoundscape, Log, All);

class SOUNDSCAPE_API FSoundscapeModule: public IModuleInterface
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};



