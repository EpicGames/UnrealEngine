// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SoundScapeModule.h"
#include "Engine/UserDefinedEnum.h"
#include "GameplayTagContainer.h"
#include "SoundScapePalette.generated.h"

class USoundscapeColor;
class UActiveSoundscapeColor;

// Struct storing Modulation State
USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapePaletteColor
{
	GENERATED_BODY()

	// Soundscape Color to Play
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette")
	TObjectPtr<USoundscapeColor> SoundscapeColor = nullptr;

	// Base Volume Scalar
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette", meta = (ClampMin = "0.0", ClampMax = "4.0", UIMin = "0.0", UIMax = "4.0", SliderExponent = "6.0"))
	float ColorVolume = 1.0f;

	// Base Pitch Scalar
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette", meta = (ClampMin = "0.2", ClampMax = "4.0", UIMin = "0.2", UIMax = "4.0", SliderExponent = "3.0"))
	float ColorPitch = 1.0f;

	// Volume Fade In Time
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float ColorFadeIn = 1.0f;

	// Volume Fade Out Time
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float ColorFadeOut = 1.0f;
};

// Class containing relevant data for a Soundscape Element
UCLASS(BlueprintType, ClassGroup = Soundscape)
class SOUNDSCAPE_API USoundscapePalette : public UObject
{
	GENERATED_BODY()

public:
	USoundscapePalette();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette")
	FGameplayTagQuery SoundscapePalettePlaybackConditions;

	// Elements
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette")
	TArray<FSoundscapePaletteColor> Colors;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject INterface
};


UCLASS(BlueprintType, ClassGroup = Soundscape)
class SOUNDSCAPE_API UActiveSoundscapePalette : public UObject
{
	GENERATED_BODY()

public:
	void InitializeSettings(UObject* WorldContextObject, USoundscapePalette* SoundscapePalette);

	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void Play();

	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void Stop();

private:
	UPROPERTY()
	TObjectPtr<UWorld> World;

	UPROPERTY()
	TArray<TObjectPtr<UActiveSoundscapeColor>> ActiveSoundscapeColors;

};
