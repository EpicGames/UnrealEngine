// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Engine/UserDefinedEnum.h"
#include "Engine/World.h"
#include "UObject/NoExportTypes.h"
#include "GameplayTagContainer.h"
#include "SoundscapeColor.generated.h"

class USoundBase;
class UAudioComponent;

#if WITH_EDITOR
/** UObject delegate to broadcast parameter changes to ActiveSoundscapeColor instances. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSoundscapeColorParameterChanges, const class USoundscapeColor*, SoundscapeColor);
#endif

UENUM(BlueprintType)
enum class ESoundscapeColorAltitudeClampMode : uint8
{
	// Altitude relative to the listener location
	Relative = 0 UMETA(DisplayName = "Relative Altitude"), 

	// Altitude based on world coordinates
	Absolute = 1 UMETA(DisplayName = "Absolute Altitude") 
};

// Struct 
USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapeColorSpawnSettings
{
	GENERATED_BODY()

	// Include an additional delay for the first spawn attempt between MinFirstSpawnDelay and MaxFirstSpawnDelay times.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior")
	bool bDelayFirstSpawn = false;

	// Sets the minimum time delay before the first spawn
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bDelayFirstSpawn"))
	float MinFirstSpawnDelay = 1.0f;

	// Sets the maximum time delay before the first spawn
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bDelayFirstSpawn"))
	float MaxFirstSpawnDelay = 3.0f;

	// When true, color will continue to attempt to spawn sounds at an update frequency between the MinSpawnDelay and MaxSpawnDelay times. When false, it will only spawn once.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior")
	bool bContinuouslyRespawn = false;

	// Sets the minimum time between respawns
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bContinuouslyRespawn"))
	float MinSpawnDelay = 1.0f;

	// Sets the maximum time between respawns
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bContinuouslyRespawn"))
	float MaxSpawnDelay = 3.0f;

	// Max number of concurrent Elements of this type playing back at once, will not Spawn New ones until current ones are Finished Playing
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "1", UIMin = "1"))
	int32 MaxNumberOfSpawnedElements = 1;

	// Sets Minimum Distance from Base for Playback of the Element
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float MinSpawnDistance = 0.0f;

	// Sets the Maximum Distance from Base for Playback of the Element
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float MaxSpawnDistance = 3500.0f;

	// Culls the voices by the Max Spawn Distance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior")
	bool bCullByMaxSpawnDistance = true;

	// The Minimum Azimuth based on the Base's Forward Vector in which to position Element Playback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", ClampMax = "180.0", UIMin = "0.0", UIMax = "180.0"))
	float MinSpawnAngle = 0.0f;

	// The Maximum Azimuth based on the Base's Forward Vector in which to position Element Playback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", ClampMax = "180.0", UIMin = "0.0", UIMax = "180.0"))
	float MaxSpawnAngle = 180.0f;

	// When true, spawn locations will clamp the z axis according to the ClampMode rules between the MinSpawnHeightClamp and the MaxSpawnHeightClamp
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior")
	bool bClampHeight = false;

	// What mode of altitude clamping to use, Relative clamps the z-axis about the location of the Listener, Absolute clamps the z-axis of the world coordinates
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (EditCondition = "bClampHeight"))
	ESoundscapeColorAltitudeClampMode ClampMode = ESoundscapeColorAltitudeClampMode::Relative;

	// After Element Playback position is determined, will clamp the Z-Axis to these ranges
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (EditCondition = "bClampHeight"))
	float MinSpawnHeightClamp = 0.0f;

	// After Element Playback position is determined, will clamp the Z-Axis to these ranges
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (EditCondition = "bClampHeight"))
	float MaxSpawnHeightClamp = 5000.0f;

	// When false, the sound rotation will be unmodified
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior")
	bool bRotateSoundSource = false;

	// The Minimum azimuthal rotation in the world orientation for Element Playback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", ClampMax = "360.0", UIMin = "0.0", UIMax = "360.0", EditCondition = "bRotateSoundSource"))
	float MinAzimuthalRotationAngle = 0.0f;

	// The Maximum azimuthal rotation in the world orientation for Element Playback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", ClampMax = "360.0", UIMin = "0.0", UIMax = "360.0", EditCondition = "bRotateSoundSource"))
	float MaxAzimuthalRotationAngle = 360.0f;

	// The Minimum Altitudinal rotation in the world orientation for Element Playback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", ClampMax = "360.0", UIMin = "0.0", UIMax = "360.0", EditCondition = "bRotateSoundSource"))
	float MinAltitudinalRotationAngle = 0.0f;

	// The Maximum Altitudinal rotation in the world orientation for Element Playback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SpawnBehavior", meta = (ClampMin = "0.0", ClampMax = "360.0", UIMin = "0.0", UIMax = "360.0", EditCondition = "bRotateSoundSource"))
	float MaxAltitudinalRotationAngle = 360.0f;

	// Position By Trace allows sounds to be repositioned from their spawn point to a new point in space based on the success of an async trace call,
	// unsuccessful hits will result in the sound not spawning
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior")
	bool bPositionByTrace = false;

	// The Minimum normalized directional vector in world orientation for the Trace to fire (default to down, z = -1.0f)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior", meta = (EditCondition = "bPositionByTrace"))
	FVector TraceVector = { 0.0f, 0.0f, -1.0f };

	// Trace distance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior", meta = (EditCondition = "bPositionByTrace"))
	float TraceDistance = 500.0f;

	// The physics trace channel to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior", meta = (EditCondition = "bPositionByTrace"))
	TEnumAsByte<enum ECollisionChannel> TraceChannel = ECollisionChannel::ECC_Visibility;

	// Filter by Color Point Density allows designers to only allow this Color to spawn in spaces with a specified range of Color Points
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior")
	bool bFilterByColorPointDensity = false;

	// Color Point
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior", meta = (EditCondition = "bFilterByColorPointDensity"))
	FGameplayTag ColorPoint;

	// Minimum number of Color Point density required before spawn is successful
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior", meta = (EditCondition = "bFilterByColorPointDensity"))
	int32 MinColorPointNumber = 1;

	// Culls the voices by the Max Spawn Distance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "SpawnBehavior", meta = (EditCondition = "bCullByMaxSpawnDistance"))
	bool bIgnoreZAxisWhenCullingByMaxDistance = true;

};

// Struct 
USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapeColorModulationSettings
{
	GENERATED_BODY()

	// An additional randomization of Volume between VolumeMin and VolumeMax on top of previous volume adjustments
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior")
	bool bRandomizeVolume = false;

	// Minimum Volume Range for Step
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.0", ClampMax = "4.0", UIMin = "0.0", UIMax = "4.0", SliderExponent = "6.0", EditCondition = "bRandomizeVolume"))
	float VolumeMin = 0.95f;

	// Maximum Volume Range for Step
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.0", ClampMax = "4.0", UIMin = "0.0", UIMax = "4.0", SliderExponent = "6.0", EditCondition = "bRandomizeVolume"))
	float VolumeMax = 1.05f;

	//  An additional randomization of Pitch between PitchMin and PitchMax on top of previous pitch adjustments
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior")
	bool bRandomizePitch = false;

	// Minimum Pitch Range for Step
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.2", ClampMax = "4.0", UIMin = "0.2", UIMax = "4.0", SliderExponent = "3.0", EditCondition = "bRandomizePitch"))
	float PitchMin = 0.95f;

	// Maximum Pitch Range for Step
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.2", ClampMax = "4.0", UIMin = "0.2", UIMax = "4.0", SliderExponent = "3.0", EditCondition = "bRandomizePitch"))
	float PitchMax = 1.05f;

	// Add Volume Fades to the Re-trigger Sound playback
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior")
	bool bFadeVolume = false;

	// This is the minimum Fade-In duration for re-triggered sounds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bFadeVolume"))
	float MinFadeInTime = 0.0f;

	// This is the maximum Fade-In duration for re-triggered sounds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bFadeVolume"))
	float MaxFadeInTime = 0.0f;

	// This is the minimum Fade-Out duration for re-triggered sounds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bFadeVolume"))
	float MinFadeOutTime = 0.0f;

	// This is the maximum Fade-Out duration for re-triggered sounds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bFadeVolume"))
	float MaxFadeOutTime = 0.0f;

	// Only Fade-In on retriggers
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ModulationBehavior", meta = (EditCondition = "bFadeVolume"))
	bool bOnlyFadeInOnRetrigger = false;

};

// Struct containing Sound Scape Element Control Data
USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapeColorPlaybackSettings
{
	GENERATED_BODY()

	// Upon starting playback, will randomly seek into the Sound if true
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PlaybackBehavior")
	bool bRandomizeStartingSeekTime = false;

	// Limit Ambient Bed's playback duration
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PlaybackBehavior")
	bool bLimitPlaybackDuration = false;

	// This is the minimum playback duration for this ambient bed (assuming a looping sound), ignored if bLimitPlaybackDuration is false
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PlaybackBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bLimitPlaybackDuration"))
	float MinPlaybackDuration = 0.0f;

	// This is the maximum playback duration for this ambient bed (assuming a looping sound), ignored if bLimitPlaybackDuration is false
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PlaybackBehavior", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "bLimitPlaybackDuration"))
	float MaxPlaybackDuration = 0.0f;

};

// New SoundColorVoice Spawning Parameters
USTRUCT()
struct SOUNDSCAPE_API FSoundscapeColorVoiceParams
{
	GENERATED_BODY()

	float Volume = 1.0f;
	float Pitch = 1.0f;
	float FadeInTime = 0.0f;
	float StartTime = 0.0f;
	FVector Location;
	FRotator Rotation;
};

/**
 * 
 */
UCLASS(BlueprintType, ClassGroup = Soundscape)
class SOUNDSCAPE_API USoundscapeColor : public UObject
{
	GENERATED_BODY()
	
public:
	USoundscapeColor();

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITORONLY_DATA
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	// Sound to Play
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Color")
	TObjectPtr<USoundBase> Sound;

	// Base Volume Scalar
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Color", meta = (ClampMin = "0.0", ClampMax = "4.0", UIMin = "0.0", UIMax = "4.0", SliderExponent = "6.0"))
	float VolumeBase = 1.0f;

	// Base Pitch Scalar
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Color", meta = (ClampMin = "0.2", ClampMax = "4.0", UIMin = "0.2", UIMax = "4.0", SliderExponent = "3.0"))
	float PitchBase = 1.0f;

	//Modulation behavior
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Color", meta = (ShowOnlyInnerProperties))
	FSoundscapeColorModulationSettings ModulationBehavior;

	// Playback behavior
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Color", meta = (ShowOnlyInnerProperties))
	FSoundscapeColorPlaybackSettings PlaybackBehavior;

	// Playback behavior
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Color", meta = (ShowOnlyInnerProperties))
	FSoundscapeColorSpawnSettings SpawnBehavior;

#if WITH_EDITORONLY_DATA
	// Delegate call for property changes
	UPROPERTY()
	FOnSoundscapeColorParameterChanges OnSoundscapeColorParameterChanges;
#endif
};

UCLASS()
class SOUNDSCAPE_API UActiveSoundscapeColorVoice : public UObject
{
	GENERATED_BODY()

public:
	// ~UObject Interface
	void BeginDestroy();
	//~UObject Interface End

	UPROPERTY(EditAnywhere, Category = "Soundscape|Color")
	TObjectPtr<UAudioComponent> AudioComponent = nullptr;

	UPROPERTY(EditAnywhere, Category = "Soundscape|Color")
	FTimerHandle TimerHandle;

	UPROPERTY(EditAnywhere, Category = "Soundscape|Color")
	bool bFree = false;

	// Limited duration timer
	UFUNCTION()
	void StopLimitedDurationVoice(float FadeOutTime);
};

UCLASS(BlueprintType, ClassGroup = Soundscape)
class SOUNDSCAPE_API UActiveSoundscapeColor : public UObject
{
	GENERATED_BODY()

public:

	// ~UObject Interface
	virtual void BeginDestroy() override;
	//~UObject Interface End
	
	// Initializes parameters based on SoundscapeColor settings
	void SetParameterValues(const USoundscapeColor* SoundscapeColor);


#if WITH_EDITOR
	// Functionality related to Settings parameter changes to feed changes in preset Asset to instances of Active Soundscape Colors
public:

	// Setup to bind to parameter change delegate
	void BindToParameterChangeDelegate(USoundscapeColor* SoundscapeColor);

	// Function bound to the Parameter change delegate 
	UFUNCTION()
	void OnSoundscapeColorParameterChange(const USoundscapeColor* SoundscapeColor);
#endif

#if WITH_EDITORONLY_DATA
	// Delegates for parameter change bindings
	FScriptDelegate SoundscapeParameterChangeDelegate;
#endif

private:
	// Sound to Play
	UPROPERTY()
	TObjectPtr<USoundBase> Sound;

	// Base Volume Scalar
	float VolumeBase = 1.0f;

	// Base Pitch Scalar
	float PitchBase = 1.0f;

	//Modulation behavior
	UPROPERTY()
	FSoundscapeColorModulationSettings ModulationBehavior;

	// Playback behavior
	UPROPERTY()
	FSoundscapeColorPlaybackSettings PlaybackBehavior;

	// Playback behavior
	UPROPERTY()
	FSoundscapeColorSpawnSettings SpawnBehavior;

public:
	// Playback start values
	float VolumeMod = 1.0f;
	float PitchMod = 1.0f;
	float FadeInMin = 1.0f;

	// Playback stop values
	float FadeOutMin = 1.0f;

	// Native play function
	void PlayNative();

	// Native stop function
	void StopNative();

	// Play Active Soundscape Color
	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void Play(float ColorVolume = 1.0f, float ColorPitch = 1.0f, float ColorFadeInTime = 1.0f);

	// Stop Active Soundscape Color
	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void Stop(float ColorFadeOutTime = 1.0f);

	// Is playing
	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	bool IsPlaying();

	// Timer Update Call
	UFUNCTION()
	void UpdateSoundscapeColor();

	// Timer delegate for amortized updates
	FTimerHandle UpdateTimer;

protected:

	// Call an async trace
	FTraceHandle SpawnSoundByTrace();

	// Function called on completion of trace
	void OnTraceCompleted(const FTraceHandle& Handle, FTraceDatum& Data);

	FTraceHandle TraceHandle;
	FTraceDelegate TraceDelegate;

private:
	// Internal begin playing and bind FTimer Delegate set first time delays
	void StartPlaying();

	// Internal stop playing, tear down FTimer delegate, etc.
	void StopPlaying();

	// Internal update call
	void Update();

	// Calculate spawning parameters
	void CalculateSpawnParams();

	// Evaluate color point density meets threshold, returns true if evaluation passes or if we are not filtering by color point density
	bool EvaluateColorPointDensity();

	// Spawn a Color Voice
	void SpawnSoundscapeColorVoice();

	// Last calculated color voice parameters
	FSoundscapeColorVoiceParams NewSoundParams;

	// Currently mid trace
	bool bTracing = false;

	// Audio Component Pool
	UPROPERTY()
	TArray<TObjectPtr<UActiveSoundscapeColorVoice>> SoundscapeColorVoicePool;

	// Is Playing Bool
	bool bIsPlaying = false;

	// First Spawn
	bool bFirstSpawn = true;

	// Voice in pool is free
	bool bVoiceInPoolFree = false;
};