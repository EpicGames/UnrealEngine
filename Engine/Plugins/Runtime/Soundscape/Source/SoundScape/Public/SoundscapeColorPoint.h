// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "GameplayTagContainer.h"
#include "SoundscapeColorPoint.generated.h"

USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapeColorPoint
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|ColorPoint")
	FGameplayTag ColorPoint;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|ColorPoint")
	FVector Location = FVector::ZeroVector;
};

USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapeColorPointVectorArray
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|ColorPoint")
	FGameplayTag ColorPoint;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|ColorPoint")
	TArray<FVector> Locations;
};

UCLASS( ClassGroup=(Soundscape), meta=(BlueprintSpawnableComponent) )
class SOUNDSCAPE_API USoundscapeColorPointComponent : public UActorComponent
{
	GENERATED_BODY()

public:	
	// Sets default values for this component's properties
	USoundscapeColorPointComponent();

	// This Component's Soundscape Pigment value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|ColorPoint", meta = (Categories = "Soundscape"))
	FGameplayTag ColorPoint;

	// Outputs information about this Active Color Point (Color Point value and Location of Parent Actor)
	void GetInfo(FGameplayTag& ColorPoint, FVector& Location) const;

protected:
	// Called when the game starts
	virtual void BeginPlay() override;

	// Called when the game ends
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

};
