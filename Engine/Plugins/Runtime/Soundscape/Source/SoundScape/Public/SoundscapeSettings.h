// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/SoftObjectPath.h"
#include "SoundscapeSettings.generated.h"

/**
 * 
 */
UCLASS(config = Game, defaultconfig, meta = (DisplayName = "Soundscape"))
class SOUNDSCAPE_API USoundscapeSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

	// Soundscape Palette Collection
	UPROPERTY(config, EditAnywhere, Category = "Soundscape", meta = (AllowedClasses = "/Script/Soundscape.SoundscapePalette"))
	TSet<FSoftObjectPath> SoundscapePaletteCollection;

	UPROPERTY(config, EditAnywhere, Category = "Soundscape")
	bool bDebugDraw = false;

	// Hash Cell Width for LOD1
	UPROPERTY(config, EditAnywhere, Category = "ColorPointSystem", meta = (ClampMin = "1.0", UIMin = "1.0"))
	float LOD1ColorPointHashWidth = 500.0f;

	// Hash Cell LOD1 Max Distance
	UPROPERTY(config, EditAnywhere, Category = "ColorPointSystem", meta = (ClampMin = "1.0", UIMin = "1.0"))
	float LOD1ColorPointHashDistance = 5000.0f;

	// Hash Cell Width for LOD2
	UPROPERTY(config, EditAnywhere, Category = "ColorPointSystem", meta = (ClampMin = "1.0", UIMin = "1.0"))
	float LOD2ColorPointHashWidth = 2500.0f;

	// Hash Cell LOD2 Max Distance
	UPROPERTY(config, EditAnywhere, Category = "ColorPointSystem", meta = (ClampMin = "1.0", UIMin = "1.0"))
	float LOD2ColorPointHashDistance = 10000.0f;

	// Hash Cell Width for LOD3
	UPROPERTY(config, EditAnywhere, Category = "ColorPointSystem", meta = (ClampMin = "1.0", UIMin = "1.0"))
	float LOD3ColorPointHashWidth = 10000.0f;

	// Hash Cell Width for the Active Hash
	UPROPERTY(config, EditAnywhere, Category = "ColorPointSystem", meta = (ClampMin = "1.0", UIMin = "1.0"))
	float ActiveColorPointHashWidth = 500.0f;

	// Hash Cell Update Timing for the Active Hash
	UPROPERTY(config, EditAnywhere, Category = "ColorPointSystem", meta = (ClampMin = "0.1", UIMin = "0.1"))
	float ActiveColorPointHashUpdateTimeSeconds = 1.0f;

public:

	// Beginning of UDeveloperSettings Interface
	virtual FName GetCategoryName() const override { return FName(TEXT("Game")); }
#if WITH_EDITOR
	virtual FText GetSectionText() const override { return NSLOCTEXT("SoundscapePlugin", "SoundscapeSettingsSection", "Soundscape"); };
#endif
	// End of UDeveloperSettings Interface


#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
