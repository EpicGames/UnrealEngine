// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "GameplayTagContainer.h"
#include "Audio.h"
#include "SoundscapeColorPoint.h"
#include "SoundscapeSubsystem.generated.h"

class USoundscapePalette;
class UActiveSoundscapePalette;
struct FStreamableHandle;

DECLARE_LOG_CATEGORY_EXTERN(LogSoundscapeSubsystem, Log, All);

// Struct 
USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapePaletteCollection
{
	GENERATED_BODY()

		// Soundscape Palette Collection
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|Palette", meta = (AllowedClasses = "/Script/Soundscape.SoundscapePalette"))
		TSet<FSoftObjectPath> SoundscapePaletteCollection;
};

// Struct 
USTRUCT()
struct SOUNDSCAPE_API FSoundscapePaletteCollectionLoaded
{
	GENERATED_BODY()

	// Soundscape Palette Collection
	UPROPERTY(EditAnywhere, Category = "Soundscape|Palette")
	TSet<TObjectPtr<USoundscapePalette>> SoundscapePaletteCollection;
};

// Struct
USTRUCT(BlueprintType)
struct SOUNDSCAPE_API FSoundscapeColorPointCollection
{
	GENERATED_BODY()

	// Soundscape Color Point Collection
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Soundscape|ColorPoint")
	TArray<FSoundscapeColorPointVectorArray> ColorPointCollection;
};

// Class
UCLASS()
class SOUNDSCAPE_API USoundscapeColorPointHashCellDensity : public UObject
{
	GENERATED_BODY()

public:

	// Soundscape Color Point Density for a Hash Cell
	UPROPERTY(EditAnywhere, Category = "Soundscape|ColorPoint")
	TMap<uint64, int32> ColorPointHashCellDensity;

};

UCLASS()
class SOUNDSCAPE_API USoundscapeColorPointHashMap : public UObject
{
	GENERATED_BODY()

public:
	// Clears and initializes hash map, sizes grid to HashCellWidth in Uunits
	UFUNCTION()
	void InitializeHash(float HashCellSizeIn, FVector GridCenterIn);

	// Clears hash map
	UFUNCTION()
	void ClearHash();

	// Returns the number of ColorPoints in a Cell containing the Location
	UFUNCTION()
	int32 NumColorPointsInCell(const FVector& Location, const FGameplayTag ColorPoint);

	// Returns true if ColorPoint added to hash, false if failed (likely location is out of Hash Bounds)
	UFUNCTION()
	bool AddColorPointToHash(const FVector& Location, const FGameplayTag ColorPoint);

	// Returns true if ColorPoint added to hash, false if failed (likely location is out of Hash Bounds)
	UFUNCTION()
	void AddColorPointArrayToHash(const TArray<FVector>& Locations, const FGameplayTag ColorPoint);

	// Calculates Hash Index
	UFUNCTION()
	uint64 CalculateHashIndex(const FVector& Location);

	// Sets ceterpoint of grid to Location
	UFUNCTION()
	void SetGridCenterpoint(const FVector& Location);

	TMap<FGameplayTag, int32> GetHashMapColorPointDensitySummary();
	
protected:
	// Returns ColorPointDensityMap from Hash Map given a Color Point
	USoundscapeColorPointHashCellDensity* GetColorPointDensityMap(FGameplayTag ColorPoint);

private:
	// Color Point Hash Map
	UPROPERTY()
	TMap<FGameplayTag, TObjectPtr<USoundscapeColorPointHashCellDensity>> ColorPointHashMap;

	// Cubic Root of the Max Value of int64 rounded down to the nearest million
	static const int32 MaxGridWidth = 2000000;
	static const int32 HalfMaxGridWidth = 1000000;

	// Hash Cell Width
	UPROPERTY()
	float HashCellSize = 500.0f;

	// MaxGridWidth / Hash Cell Size
	UPROPERTY()
	int32 GridWidth = 4000;

	// Grid Width Cubed
	UPROPERTY()
	int64 NumCells = 64000000000;

	// Cell Size Inverted
	UPROPERTY()
	float HashCellFactor = 0.002f;

	// Current Grid Centerpoint
	UPROPERTY()
	FVector GridCenter;

	// Grid Origin Offset
	UPROPERTY()
	FVector GridOriginOffset;

public:
	const FVector GetHashCellCenterpoint(FVector Location);
	const float GetHashCellWidth();
};

UENUM()
enum class ESoundscapeLOD : uint8
{
	LOD1 = 0 UMETA(DisplayName = "LOD 1"),
	LOD2 UMETA(DisplayName = "LOD 2"),
	LOD3 UMETA(DisplayName = "LOD 3")
};

UCLASS()
class SOUNDSCAPE_API USoundscapeColorPointHashMapCollection : public UObject
{
	GENERATED_BODY()

public:

	void InitializeCollection();

	void AddColorPointArrayToHashMapCollection(const TArray<FVector>& Locations, const FGameplayTag ColorPoint);

	void ClearColorPointHashMapCollection();

	int32 GetColorPointHashMapCollectionDensity(const FVector Location, const FGameplayTag ColorPoint, const ESoundscapeLOD SoundscapeLOD);

public:
	void CalculateTotalColorPointDensity(TMap<FGameplayTag, int32>& TotalHashMapColorPointDensity, ESoundscapeLOD SoundscapeLOD);

private:

	// Color Point Hash Maps
	UPROPERTY()
	TObjectPtr<USoundscapeColorPointHashMap> ColorPointHashMapLOD1;
	UPROPERTY()
	TObjectPtr<USoundscapeColorPointHashMap> ColorPointHashMapLOD2;
	UPROPERTY()
	TObjectPtr<USoundscapeColorPointHashMap> ColorPointHashMapLOD3;

	// Hash Cell Width for LOD1
	UPROPERTY()
	float LOD1ColorPointHashWidth = 500.0f;

	// Hash Cell LOD1 Max Distance
	UPROPERTY()
	float LOD1ColorPointHashDistance = 5000.0f;

	// Hash Cell Width for LOD2
	UPROPERTY()
	float LOD2ColorPointHashWidth = 2500.0f;

	// Hash Cell LOD2 Max Distance
	UPROPERTY()
	float LOD2ColorPointHashDistance = 10000.0f;

	// Hash Cell Width for LOD3
	UPROPERTY()
	float LOD3ColorPointHashWidth = 10000.0f;

};

/**
 * 
 */
UCLASS()
class SOUNDSCAPE_API USoundscapeSubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()
	
public:
	// Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	// End USubsystem


	// Settings
public:

	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void SetState(FGameplayTag SoundscapeState);

	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void ClearState(FGameplayTag SoundscapeState);

	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void RestartSoundscape();

	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	bool AddPaletteCollection(FName PaletteCollectionName, FSoundscapePaletteCollection PaletteCollection);

	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	bool RemovePaletteCollection(FName PaletteCollectionName);

	bool bDebugMode = false;

private:
	UPROPERTY()
	TSet<TObjectPtr<USoundscapePalette>> LoadedPaletteCollectionSet;

	UPROPERTY()
	TMap<FName, FSoundscapePaletteCollection> UnloadedPaletteCollections;

	FGameplayTagContainer SubsystemState;

	UPROPERTY()
	TMap<TObjectPtr<USoundscapePalette>, TObjectPtr<UActiveSoundscapePalette>> ActivePalettes;

	// Start Asynchronous Loading 
	bool LoadPaletteCollection(FName PaletteCollectionName);
	void CompleteAsyncLoading(FName PaletteCollectionName);

	TMap<FName, TSharedPtr<FStreamableHandle>> ActiveAssetLoadHandles;

	bool UnloadPaletteCollection(FName PaletteCollectionName);

	void UpdateState();

private:

	Audio::FDeviceId AudioDeviceID;

	// Soundscape Color Point system
public:
	// Add a Color Point Collection to the Subsystem, returns true if successful
	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	void AddColorPointCollection(FName ColorPointCollectionName, FSoundscapeColorPointCollection ColorPointCollection);

	// Remove a Color Point Collection from the Subsystem, returns true if successful
	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	bool RemoveColorPointCollection(FName ColorPointCollectionName);

	// Check Color Point Density for a Location Cell
	UFUNCTION(BlueprintCallable, Category = "Soundscape")
	int32 CheckColorPointDensity(FVector Location, FGameplayTag ColorPoint);

	// Add Hash Map Collection
	void AddColorPointHashMapCollection(USoundscapeColorPointHashMapCollection* ColorPointHashMapCollection);

	// Add Hash Map Collection
	bool RemoveColorPointHashMapCollection(USoundscapeColorPointHashMapCollection* ColorPointHashMapCollection);

	// Draw Debug Cell
	void DrawDebugCell(FVector Location, bool bSuccess);

	// Calculate Debug Cell Dimensions
	TPair<FVector, FVector> CalculateDebugCellDimensions(FVector Location, ESoundscapeLOD SoundscapeLOD);

	// Add Active Color Point
	void AddActiveColorPoint(const USoundscapeColorPointComponent* SoundscapeColorPointComponent);

	// Remove Active Color Point
	void RemoveActiveColorPoint(const USoundscapeColorPointComponent* SoundscapeColorPointComponent);

private:
	// Stored Hash Map Collections
	UPROPERTY(Transient)
	TArray<TObjectPtr<USoundscapeColorPointHashMapCollection>> ColorPointHashMapCollections;

	// Stored Color Point Collections
	TMap<FName, FSoundscapeColorPointCollection> ColorPointCollections;

	// Active Color Point Collection
	FSoundscapeColorPointCollection ActiveColorPointCollection;

	// Color Point Hash Maps
	UPROPERTY()
	TObjectPtr<USoundscapeColorPointHashMap> ColorPointHashMapLOD1;
	UPROPERTY()
	TObjectPtr<USoundscapeColorPointHashMap> ColorPointHashMapLOD2;
	UPROPERTY()
	TObjectPtr<USoundscapeColorPointHashMap> ColorPointHashMapLOD3;

	UPROPERTY()
	TObjectPtr<USoundscapeColorPointHashMap> ActiveColorPointHashMap;

	// Hash Cell Width for LOD1
	float LOD1ColorPointHashWidth = 500.0f;

	// Hash Cell LOD1 Max Distance
	float LOD1ColorPointHashDistance = 5000.0f;

	// Hash Cell Width for LOD2
	float LOD2ColorPointHashWidth = 2500.0f;

	// Hash Cell LOD2 Max Distance
	float LOD2ColorPointHashDistance = 10000.0f;

	// Hash Cell Width for LOD3
	float LOD3ColorPointHashWidth = 10000.0f;

	// Hash Cell Width for the Active Hash
	float ActiveColorPointHashWidth = 500.0f;

	// Hash Cell Update Timing for the Active Hash
	float ActiveColorPointHashUpdateTimeSeconds = 1.0f;

	// Update Hash Map
	void UpdateColorPointHashMap(USoundscapeColorPointHashMap& SoundscapeColorPointHashMap);

	// Update Active Hash Map
	void UpdateActiveColorPointHashMap();

	// Timer delegate for amortized updates
	FTimerHandle ActiveColorPointUpdateTimer;
	
	// Active Color Point Components
	TArray<const USoundscapeColorPointComponent*> ActiveSoundscapeColorPointComponents;
};
