// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Soundscape : ModuleRules
	{
        public Soundscape(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"GameplayTags",
					// ... add other public dependencies that you statically link with here ...
				}
				);


			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				"CoreUObject",
				"Engine",
					// ... add private dependencies that you statically link with here ...	
				}
				);
		}
	}
}