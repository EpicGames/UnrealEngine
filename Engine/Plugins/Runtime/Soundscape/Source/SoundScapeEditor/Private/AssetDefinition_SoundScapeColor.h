// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "SoundscapeColor.h"
#include "AssetDefinition_SoundScapeColor.generated.h"

UCLASS()
class UAssetDefinition_SoundScapeColor : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundscapeColor", "Soundscape Color"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(0, 175, 175); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundscapeColor::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Audio,
					NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundscapeColorSubMenu", "Advanced"),
					FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundscapeColorSubMenuSection", "SoundScape"), ECategoryMenuType::Section))
			};
		return Categories;
	}
	// UAssetDefinition End
};
