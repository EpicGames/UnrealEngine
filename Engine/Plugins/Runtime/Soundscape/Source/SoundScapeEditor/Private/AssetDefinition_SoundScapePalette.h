// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "SoundScapePalette.h"
#include "AssetDefinition_SoundScapePalette.generated.h"

UCLASS()
class UAssetDefinition_SoundScapePalette : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundScapePalette", "Soundscape Palette"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(0, 125, 255); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USoundscapePalette::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundScapePaletteSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_SoundScapePaletteSubMenuSection", "SoundScape"), ECategoryMenuType::Section))
		};
		return Categories;
	}
	// UAssetDefinition End
};
