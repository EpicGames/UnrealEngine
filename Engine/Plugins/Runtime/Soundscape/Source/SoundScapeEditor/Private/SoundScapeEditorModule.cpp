// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundScapeEditorModule.h"

IMPLEMENT_MODULE(FSoundscapeEditorModule, SoundscapeEditor)

void FSoundscapeEditorModule::StartupModule()
{
}

void FSoundscapeEditorModule::ShutdownModule()
{
}
