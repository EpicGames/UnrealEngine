// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundScapePaletteFactory.h"
#include "SoundScapePalette.h"

USoundscapePaletteFactory::USoundscapePaletteFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundscapePalette::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundscapePaletteFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundscapePalette* SoundscapePalette = NewObject<USoundscapePalette>(InParent, Name, Flags);

	return SoundscapePalette;
}
