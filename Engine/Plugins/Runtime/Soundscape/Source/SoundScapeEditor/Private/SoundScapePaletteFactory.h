// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "SoundScapePaletteFactory.generated.h"

UCLASS(hidecategories = Object, MinimalAPI)
class USoundscapePaletteFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;

	virtual bool ShouldShowInNewMenu() const override
	{
		return true;
	}
	//~ End UFactory Interface	
};

