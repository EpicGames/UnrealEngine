// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundscapeColorFactory.h"
#include "SoundscapeColor.h"

USoundscapeColorFactory::USoundscapeColorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundscapeColor::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USoundscapeColorFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USoundscapeColor* SoundscapeColor = NewObject<USoundscapeColor>(InParent, Name, Flags);

	return SoundscapeColor;
}