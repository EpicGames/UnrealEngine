// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SoundscapeEditor : ModuleRules
	{
		public SoundscapeEditor(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				"Core",
				"Soundscape",
				"AudioMixer",

					// ... add other public dependencies that you statically link with here ...
				}
				);


			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				"AssetDefinition",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"UnrealEd",
					// ... add private dependencies that you statically link with here ...	
				}
				);
		}
	}
}